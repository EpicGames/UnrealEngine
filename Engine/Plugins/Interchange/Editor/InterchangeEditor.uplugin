{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange Editor",
	"Description": "The Interchange Editor plugin exposes the Interchange import framework and pipelines to Unreal Editor.",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "InterchangeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		},
		{
			"Name": "InterchangeEditorPipelines",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		},
		{
			"Name": "InterchangeEditorUtilities",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		}
	],
	"Plugins": [
		{
			"Name": "Interchange",
			"Enabled": true,
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		}
	],
	"IsExperimentalVersion": false
}