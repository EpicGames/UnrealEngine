// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeEditor : ModuleRules
	{
		public InterchangeEditor(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
					"CoreUObject",
					"Engine",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"AssetTools",
					"ContentBrowser",
					"EditorSubsystem",
					"InterchangeCommonParser",
					"InterchangeCore",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeFbxParser",
					"InterchangeImport",
					"InterchangeNodes",
					"InterchangePipelines",
					"LevelEditor",
					"MessageLog",
					"Slate",
					"SlateCore",
					"InputCore", // Translator settings customizations
					"Slate", // Translator settings customizations
					"ToolMenus",
				}
			);
		}
    }
}
