// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_InterchangeSceneImportAsset.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeAssetUserData.h"
#include "InterchangeEditorModule.h"
#include "InterchangeImportReset.h"
#include "InterchangeLevelFactoryNode.h"
#include "InterchangeManager.h"
#include "InterchangeEditorScriptLibrary.h"
#include "InterchangeSceneImportAsset.h"

#include "ContentBrowserMenuContexts.h"
#include "EditorReimportHandler.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/IConsoleManager.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "Misc/App.h"
#include "Misc/DelayedAutoRegister.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_InterchangeSceneImportAsset)

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
extern UNREALED_API UEditorEngine* GEditor;
#endif

#define LOCTEXT_NAMESPACE "AssetDefinition_InterchangeSceneImportAsset"

FAssetCategoryPath UAssetDefinition_InterchangeSceneImportAsset::Interchange(LOCTEXT("Interchange_Category_Path", "Interchange"));

EAssetCommandResult UAssetDefinition_InterchangeSceneImportAsset::OpenAssets(const FAssetOpenArgs& /*OpenArgs*/) const
{
	return EAssetCommandResult::Handled;
}

UThumbnailInfo* UAssetDefinition_InterchangeSceneImportAsset::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

namespace MenuExtension_InterchangeSceneImportAsset
{

	void ExecuteResetScene(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UInterchangeSceneImportAsset*> SceneImportAssets = Context->LoadSelectedObjects<UInterchangeSceneImportAsset>();

		for (UInterchangeSceneImportAsset* SceneImportAsset : SceneImportAssets)
		{
			if (SceneImportAsset)
			{
				UInterchangeEditorScriptLibrary::ResetSceneImportAsset(SceneImportAsset);
			}
		}
	}

	void ExecuteReimportOneAsset(UInterchangeSceneImportAsset* Asset, const FString& FilePath)
	{
		using namespace UE::Interchange;

		FScopedSourceData ScopedSourceData(FilePath);
		const UInterchangeSourceData* SourceData = ScopedSourceData.GetSourceData();

		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		if (InterchangeManager.CanTranslateSourceData(SourceData))
		{
			FImportAssetParameters ImportAssetParameters;
			ImportAssetParameters.bIsAutomated = GIsAutomationTesting || FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript;
			ImportAssetParameters.ReimportAsset = Asset;
			ImportAssetParameters.ReimportSourceIndex = INDEX_NONE;
			ImportAssetParameters.ImportLevel = nullptr;

			TTuple<FAssetImportResultRef, FSceneImportResultRef> ImportResult = InterchangeManager.ImportSceneAsync(FString(), SourceData, ImportAssetParameters);

			// TODO: Report if reimport failed
			//return ImportResult;
		}
	}

	void ExecuteReimport(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UInterchangeSceneImportAsset*> SceneImportAssets = Context->LoadSelectedObjects<UInterchangeSceneImportAsset>();

		for (UInterchangeSceneImportAsset* SceneImportAsset : SceneImportAssets)
		{
			if (SceneImportAsset)
			{
				ExecuteReimportOneAsset(SceneImportAsset, SceneImportAsset->AssetImportData->GetFirstFilename());
			}
		}
	}

	void ExecuteReimportWithFile(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UInterchangeSceneImportAsset*> SceneImportAssets = Context->LoadSelectedObjects<UInterchangeSceneImportAsset>();
		if (SceneImportAssets.Num() != 1 || !SceneImportAssets[0])
		{
			return;
		}

#if WITH_EDITORONLY_DATA
		UInterchangeSceneImportAsset* SceneImportAsset = SceneImportAssets[0];
		if (!SceneImportAsset->AssetImportData)
		{
			return;
		}

		TArray<FString> OpenFilenames = SceneImportAsset->AssetImportData->ExtractFilenames();
		FReimportManager::Instance()->GetNewReimportPath(SceneImportAsset, OpenFilenames);
		if (OpenFilenames.Num() == 1 && !OpenFilenames[0].IsEmpty())
		{
			ExecuteReimportOneAsset(SceneImportAsset, OpenFilenames[0]);
		}
#endif
	}

	bool CanExecuteResetScene(const FToolMenuContext& InContext)
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.InterchangeReset"), false);
		const bool bInterchangeResetEnabled = CVar ? CVar->GetBool() : false;
		if (!bInterchangeResetEnabled)
		{
			return false;
		}

		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UInterchangeSceneImportAsset*> SceneImportAssets = Context->LoadSelectedObjects<UInterchangeSceneImportAsset>();

		for (UInterchangeSceneImportAsset* SceneImportAsset : SceneImportAssets)
		{
			if (SceneImportAsset && SceneImportAsset->AssetImportData)
			{
				if (SceneImportAsset->AssetImportData->ExtractFilenames().Num() > 0)
				{
					return true;
				}
			}
		}

		return false;
	}

	bool CanExecuteReimport(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UInterchangeSceneImportAsset*> SceneImportAssets = Context->LoadSelectedObjects<UInterchangeSceneImportAsset>();

		for (UInterchangeSceneImportAsset* SceneImportAsset : SceneImportAssets)
		{
			if (SceneImportAsset && SceneImportAsset->AssetImportData)
			{
				// TODO: Check that at least one file exists before returning true
				if (SceneImportAsset->AssetImportData->ExtractFilenames().Num() > 0)
				{
					return true;
				}
			}
		}

		return false;
	}

	bool CanExecuteReimportWithFile(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UInterchangeSceneImportAsset*> SceneImportAssets = Context->LoadSelectedObjects<UInterchangeSceneImportAsset>();

		if (SceneImportAssets.Num() == 1)
		{
			const UInterchangeSceneImportAsset* SceneImportAsset = SceneImportAssets[0];
			if (SceneImportAsset && SceneImportAsset->AssetImportData)
			{
				return true;
			}
		}

		return false;
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
			{
				FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UInterchangeSceneImportAsset::StaticClass());

				if (!Menu->FindSection("Interchange"))
				{
					const FToolMenuInsert MenuInsert(NAME_None, EToolMenuInsertType::First);
					Menu->AddSection("Interchange", LOCTEXT("InterchangeSceneImportAsset_Section", "Interchange"), MenuInsert);
				}
				FToolMenuSection& Section = Menu->FindOrAddSection("Interchange");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.InterchangeReset"), false);
						const bool bInterchangeResetEnabled = CVar ? CVar->GetBool() : false;
						if (bInterchangeResetEnabled)
						{
							const TAttribute<FText> Label = LOCTEXT("InterchangeSceneImportAsset_ResetScene", "Reset Scene");
							const TAttribute<FText> ToolTip = LOCTEXT("InterchangeSceneImportAsset_ResetSceneTooltip", "Reset the scene associated with each selected InterchangeSceneImportAsset to original pipeline properties.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteResetScene);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteResetScene);

							InSection.AddMenuEntry("InterchangeSceneImportAsset_ResetScene", Label, ToolTip, Icon, UIAction);
						}

						{
							const TAttribute<FText> Label = LOCTEXT("InterchangeSceneImportAsset_Reimport", "Reimport Scene");
							const TAttribute<FText> ToolTip = LOCTEXT("InterchangeSceneImportAsset_ReimportTooltip", "Reimport the scene associated with each selected InterchangeSceneImportAsset.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteReimport);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteReimport);

							InSection.AddMenuEntry("InterchangeSceneImportAsset_Reimport", Label, ToolTip, Icon, UIAction);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("InterchangeSceneImportAsset_ReimportWithFile", "Reimport Scene With File");
							const TAttribute<FText> ToolTip = LOCTEXT("InterchangeSceneImportAsset_ReimportWithFile_Tooltip", "Reimport the scene associated with the selected InterchangeSceneImportAsset using a new file.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteReimportWithFile);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteReimportWithFile);

							InSection.AddMenuEntry("InterchangeSceneImportAsset_ReimportWithFile", Label, ToolTip, Icon, UIAction);
						}
					}));
			}));
		});
}

#undef LOCTEXT_NAMESPACE
