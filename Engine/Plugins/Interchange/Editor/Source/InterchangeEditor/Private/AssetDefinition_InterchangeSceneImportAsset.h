// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "InterchangeSceneImportAsset.h"

#include "AssetDefinition_InterchangeSceneImportAsset.generated.h"

UCLASS()
class UAssetDefinition_InterchangeSceneImportAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InterchangeSceneImportAsset", "InterchangeSceneImportAsset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInterchangeSceneImportAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { Interchange };
		return Categories;
	}
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End

private:
	static FAssetCategoryPath Interchange;
};
