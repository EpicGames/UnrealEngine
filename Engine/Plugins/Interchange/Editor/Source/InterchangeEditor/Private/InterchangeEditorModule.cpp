// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeEditorModule.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "Engine/Engine.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "HAL/IConsoleManager.h"
#include "IMessageLogListing.h"
#include "InterchangeEditorLog.h"
#include "InterchangeFbxAssetImportDataConverter.h"
#include "InterchangeManager.h"
#include "InterchangeResetContextMenuExtender.h"
#include "InterchangePipelineSettingsCacheHandler.h"
#include "IPackageAutoSaver.h"
#include "LevelEditorSubsystem.h"
#include "MessageLogModule.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UnrealEdGlobals.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "InterchangeEditorModule"

DEFINE_LOG_CATEGORY(LogInterchangeEditor);

static bool GInterchangeSyncToBrowser = true;
static FAutoConsoleVariableRef CCvarInterchangeSyncToBrowser(
	TEXT("Interchange.FeatureFlags.Import.SyncToBrowser"),
	GInterchangeSyncToBrowser,
	TEXT("Whether to select the imported assets inside the content browser once the import is done. Disabling this option can prevent the editor from freezing when importing a large number of assets."),
	ECVF_Default);

namespace UE::Interchange::InterchangeEditorModule
{
	bool HasErrorsOrWarnings(TStrongObjectPtr<UInterchangeResultsContainer> InResultsContainer)
	{
		for (UInterchangeResult* Result : InResultsContainer->GetResults())
		{
			if (Result->GetResultType() != EInterchangeResultType::Success)
			{
				return true;
			}
		}

		return false;
	}

	void LogErrors(TStrongObjectPtr<UInterchangeResultsContainer> InResultsContainer)
	{
		// Only showing when we have errors or warnings for now
		if (FApp::IsUnattended() || !HasErrorsOrWarnings(InResultsContainer))
		{
			return;
		}

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		TSharedPtr<IMessageLogListing> LogListing = MessageLogModule.GetLogListing(FName("Interchange"));

		if (ensure(LogListing))
		{
			const FText LogListingLabel = NSLOCTEXT("InterchangeImport", "Label", "Interchange Import");
			LogListing->SetLabel(LogListingLabel);

			TArray<TSharedRef<FTokenizedMessage>> TokenizedMessages;
			bool bCurrentImportHasErrors = false;

			for (UInterchangeResult* Result : InResultsContainer->GetResults())
			{
				const EInterchangeResultType ResultType = Result->GetResultType();
				if (ResultType != EInterchangeResultType::Success)
				{
					TokenizedMessages.Add(FTokenizedMessage::Create(ResultType == EInterchangeResultType::Error ? EMessageSeverity::Error : EMessageSeverity::Warning, Result->GetMessageLogText()));
					bCurrentImportHasErrors |= ResultType == EInterchangeResultType::Error;
				}
			}

			// UInterchangeManager::ReleaseAsyncHelper has output logs turned on. To avoid duplication, turning it off here.
			const bool bMirrorToOutputLog = GIsAutomationTesting;
			LogListing->AddMessages(TokenizedMessages, bMirrorToOutputLog);

			if (bCurrentImportHasErrors)
			{
				LogListing->NotifyIfAnyMessages(NSLOCTEXT("Interchange", "LogAndNotify", "There were issues with the import."), EMessageSeverity::Info);
			}
		}
	}

	void ImportStarted()
	{
		// Disable autosaving while the Interchange is in progress.	
		if (GUnrealEd)
		{
			GUnrealEd->GetPackageAutoSaver().SuspendAutoSave();
		}
	}

	void ImportFinished()
	{
		//Reinstate AutoSave
		if (GUnrealEd)
		{
			GUnrealEd->GetPackageAutoSaver().ResumeAutoSave();
		}
	}


	FConsoleVariableSinkHandle InterchangeResetCVarSinkHandle;
	void HandleInterchangeResetEnabledConsoleVarChanged()
	{
		if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.InterchangeReset"), false))
		{
			if (CVar->GetBool())
			{
				FInterchangeResetContextMenuExtender::SetupLevelEditorContextMenuExtender();
			}
			else
			{
				FInterchangeResetContextMenuExtender::RemoveLevelEditorContextMenuExtender();
			}
		}
	}

	void ImportedObjectAvailable(TMap<int32, TArray<TObjectPtr<UObject>>> ImportedAssetsPerSourceIndex, TMap<int32, TArray<TObjectPtr<UObject>>> ImportedSceneObjectsPerSourceIndex)
	{
		if (GIsAutomationTesting)
		{
			return;
		}

		bool bHasHiddenActors = false;
		for (const TPair<int32, TArray<TObjectPtr<UObject>>>& SourceIndexSceneObjectsPair : ImportedSceneObjectsPerSourceIndex)
		{
			for (const TObjectPtr<UObject>& SceneObject : SourceIndexSceneObjectsPair.Value)
			{
				if (AActor* Actor = Cast<AActor>(SceneObject.Get()))
				{
					bHasHiddenActors |= Actor->IsHidden();
				}
			}
		}

		if (bHasHiddenActors)
		{
			if (ensure(GEditor))
			{
				if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
				{
					if (!LevelEditorSubsystem->EditorGetGameView())
					{
						FText ShortcutText = FText::GetEmpty();

						const TSharedPtr<FUICommandInfo> ToggleGameViewCommand = FInputBindingManager::Get().FindCommandInContext(TEXT("LevelViewport"), TEXT("ToggleGameView"));
						if (ensure(ToggleGameViewCommand.IsValid()))
						{
							if (FText InputChordText = ToggleGameViewCommand->GetFirstValidChord()->GetInputText(); 
								!InputChordText.IsEmpty())
							{
								ShortcutText = FText::FormatOrdered(LOCTEXT("GameViewShortcutText", ", Shortcut: {0}"), InputChordText);
							}
						}

						FNotificationInfo Info(FText::GetEmpty());
						Info.Text = TAttribute<FText>::CreateLambda([ShortcutText]()
						{
							if (GEditor)
							{
								if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
								{
									if (LevelEditorSubsystem->EditorGetGameView())
									{
										return LOCTEXT("GameViewEnabledLabel", "Game View Enabled");
									}
								}
							}

							return FText::FormatOrdered(
								LOCTEXT("InvisibleActorsMessage", "Interchange imported actors that are invisible that can be visualized in the level viewport using the Game View. [Path: Viewport Toolbar > Perspective > Game View.{0}]"),
								ShortcutText
							);
						});
						Info.ExpireDuration = 6.0f;

						FNotificationButtonInfo EnableGameViewButtonInfo(
							LOCTEXT("EnableGameViewButtonLabel", "Enable Game View"),
							LOCTEXT("EnableGameViewButtonTooltipLabel", "Enables Game View to correctly reflect the visibility states of the actors in the scene."),
							FSimpleDelegate::CreateLambda([]()
							{
								if (GEditor)
								{
									if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
									{
										LevelEditorSubsystem->EditorSetGameView(true);
									}
								}
							}),
							FNotificationButtonInfo::FVisibilityDelegate::CreateLambda([](SNotificationItem::ECompletionState)->EVisibility
							{
								if (GEditor)
								{
									if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
									{
										return	(!LevelEditorSubsystem->EditorGetGameView() ? EVisibility::Visible : EVisibility::Collapsed);
									}
								}
								return EVisibility::Visible;
							}),
							FNotificationButtonInfo::FIsEnabledDelegate::CreateLambda([](SNotificationItem::ECompletionState)->bool
							{
								if(GEditor)
								{ 
									if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
									{
										return !LevelEditorSubsystem->EditorGetGameView();
									}
								}
								return false;
							})
						);
						Info.ButtonDetails.Add(EnableGameViewButtonInfo);
						FSlateNotificationManager::Get().AddNotification(Info);
					}
				}
			}
		}
	}

} //ns UE::Interchange::InterchangeEditorModule

FInterchangeEditorModule& FInterchangeEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked< FInterchangeEditorModule >(INTERCHANGEEDITOR_MODULE_NAME);
}

bool FInterchangeEditorModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(INTERCHANGEEDITOR_MODULE_NAME);
}

void FInterchangeEditorModule::StartupModule()
{
	if (IsRunningCookCommandlet())
	{
		return;
	}

	using namespace UE::Interchange;

	auto RegisterItems = [this]()
	{
		FDelegateHandle OnBatchImportCompleteHandle;
		FDelegateHandle OnImportStartedHandle;
		FDelegateHandle OnImportedObjectsAvailableHandle;
		FDelegateHandle OnImportFinishedHandle;
		FDelegateHandle OnSanitizeNameHandle;

		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		OnBatchImportCompleteHandle = InterchangeManager.OnBatchImportComplete.AddStatic(&InterchangeEditorModule::LogErrors);
		OnImportStartedHandle = InterchangeManager.OnImportStarted.AddStatic(&InterchangeEditorModule::ImportStarted);
		OnImportedObjectsAvailableHandle = InterchangeManager.OnImportedObjectsAvailable.AddStatic(&InterchangeEditorModule::ImportedObjectAvailable);
		OnImportFinishedHandle = InterchangeManager.OnImportFinished.AddStatic(&InterchangeEditorModule::ImportFinished);
		InterchangeManager.RegisterImportDataConverter(UInterchangeFbxAssetImportDataConverter::StaticClass());

		OnSanitizeNameHandle= InterchangeManager.OnSanitizeName.AddLambda([](FString& SanitizeName, const ESanitizeNameTypeFlags NameType)
			{
				//Call the asset tools sanitize
				IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
				AssetTools.SanitizeName(SanitizeName);
			});

		auto UnregisterItems = [
			OnBatchImportCompleteHandle,
			OnImportStartedHandle,
			OnImportedObjectsAvailableHandle,
			OnImportFinishedHandle,
			OnSanitizeNameHandle]()
		{
			UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
			InterchangeManager.OnBatchImportComplete.Remove(OnBatchImportCompleteHandle);
			InterchangeManager.OnImportStarted.Remove(OnImportStartedHandle);
			InterchangeManager.OnImportedObjectsAvailable.Remove(OnImportedObjectsAvailableHandle);
			InterchangeManager.OnImportFinished.Remove(OnImportFinishedHandle);
			InterchangeManager.OnSanitizeName.Remove(OnSanitizeNameHandle);
		};

		InterchangeManager.OnPreDestroyInterchangeManager.AddLambda(UnregisterItems);
		FInterchangePipelineSettingsCacheHandler::InitializeCacheHandler();

		{
			using namespace UE::Interchange::InterchangeEditorModule;
			// Register a Console Variable sink to monitor changes in Interchange.FeatureFlags.InterchangeReset
			InterchangeResetCVarSinkHandle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateStatic(&HandleInterchangeResetEnabledConsoleVarChanged));
		}
	};

	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}

	FInterchangeResetContextMenuExtender::SetupLevelEditorContextMenuExtender();
	FCoreDelegates::OnPreExit.AddStatic(FInterchangePipelineSettingsCacheHandler::ShutdownCacheHandler);
}

void FInterchangeEditorModule::ShutdownModule()
{
	FInterchangeResetContextMenuExtender::RemoveLevelEditorContextMenuExtender();

	{
		using namespace UE::Interchange::InterchangeEditorModule;
		IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(InterchangeResetCVarSinkHandle);
	}
}

IMPLEMENT_MODULE(FInterchangeEditorModule, InterchangeEditor)

#undef LOCTEXT_NAMESPACE // "InterchangeEditorModule"

