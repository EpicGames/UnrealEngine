// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeEditorScriptLibrary.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeAssetUserData.h"
#include "InterchangeImportReset.h"
#include "InterchangeLevelFactoryNode.h"
#include "InterchangeSceneImportAsset.h"

#include "Editor.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"

#include "GameFramework/WorldSettings.h"
#include "GameFramework/Actor.h"

#include "HAL/IConsoleManager.h"

#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

#include "Subsystems/EditorActorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeEditorScriptLibrary)


#define LOCTEXT_NAMESPACE "InterchangeEditorScriptLibrary"

namespace UE::InterchangeEditorScriptLibrary::Private
{
	namespace InterchangeReset
	{
		bool IsInterchangeResetAvailable()
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.InterchangeReset"), false);
			return CVar ? CVar->GetBool() : false;
		}

		ALevelInstance* SpawnLevelInstanceInEditor(FName AssetName)
		{
			FWorldContext& EditorWorldContext = GEditor->GetEditorWorldContext();
			if (UWorld* EditorWorld = EditorWorldContext.World())
			{
				FActorSpawnParameters SpawnParameters;
				SpawnParameters.Name = AssetName;
				SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
				SpawnParameters.OverrideLevel = EditorWorld->PersistentLevel;
				SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

				return EditorWorld->SpawnActor<ALevelInstance>(ALevelInstance::StaticClass(), SpawnParameters);
			}

			return nullptr;
		}

		void InitializeLevelInstance(ALevelInstance* LevelInstanceActor, UWorld* ReferencedWorld)
		{
			if (!LevelInstanceActor->GetRootComponent())
			{
				USceneComponent* RootComponent = NewObject<USceneComponent>(LevelInstanceActor, USceneComponent::GetDefaultSceneRootVariableName(), RF_Transactional);
#if WITH_EDITORONLY_DATA
				RootComponent->bVisualizeComponent = true;
#endif
				LevelInstanceActor->SetRootComponent(RootComponent);
				LevelInstanceActor->AddInstanceComponent(RootComponent);
			}

#if WITH_EDITORONLY_DATA
			UWorld* ParentWorld = LevelInstanceActor->GetWorld();

			// Make sure newly created level asset gets scanned
			ULevel::ScanLevelAssets(ReferencedWorld->GetPackage()->GetName());

			ParentWorld->PreEditChange(nullptr);

			LevelInstanceActor->SetWorldAsset(ReferencedWorld);
			LevelInstanceActor->UpdateLevelInstanceFromWorldAsset();
			LevelInstanceActor->LoadLevelInstance();

			//Reference world must be cleanup since they are not the main world.
			//This remove all the world managers and prevent GC issue when unloading the main world referencing this world.
			if (ReferencedWorld->bIsWorldInitialized)
			{
				ReferencedWorld->CleanupWorld();
			}

			ParentWorld->PostEditChange();
#endif
		}

		UInterchangeSceneImportAsset* GetSceneImportAssetFromPath(const FSoftObjectPath& SoftObjectPath)
		{
			if (UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(SoftObjectPath.TryLoad()))
			{
				return SceneImportAsset;
			}

			return nullptr;
		}


		UInterchangeSceneImportAsset* GetSceneImportAssetForWorld(const UWorld* ReferencedWorld)
		{
			if (AWorldSettings* WorldSettings = ReferencedWorld->GetWorldSettings())
			{
				if (UInterchangeLevelAssetUserData* AssetUserData = Cast<UInterchangeLevelAssetUserData>(WorldSettings->GetAssetUserDataOfClass(TSubclassOf<UAssetUserData>(UInterchangeLevelAssetUserData::StaticClass()))))
				{
					if (!AssetUserData->SceneImportPaths.IsEmpty())
					{
						return GetSceneImportAssetFromPath(FSoftObjectPath(AssetUserData->SceneImportPaths[0]));
					}
				}
			}

			return nullptr;
		}

		UInterchangeSceneImportAsset* GetSceneImportAsset(AActor* ImportedActor)
		{
			if (const ALevelInstance* LevelInstanceActor = Cast<ALevelInstance>(ImportedActor))
			{
				const TSoftObjectPtr<UWorld>& WorldObjectPtr = LevelInstanceActor->GetWorldAsset();
				if (const UWorld* ReferencedWorld = Cast<UWorld>(WorldObjectPtr.ToSoftObjectPath().TryLoad()))
				{
					if (UInterchangeSceneImportAsset* SceneImportAsset = GetSceneImportAssetForWorld(ReferencedWorld))
					{
						return SceneImportAsset;
					}
				}
			}
			else
			{
				const TSubclassOf<UAssetUserData> InterchangeAssetUserDataClass(UInterchangeAssetUserData::StaticClass());

				for (UActorComponent* Component : ImportedActor->GetComponents())
				{
					if (UInterchangeAssetUserData* AssetUserData = Cast<UInterchangeAssetUserData>(Component->GetAssetUserDataOfClass(InterchangeAssetUserDataClass)))
					{
						using namespace UE::Interchange::InterchangeReset;
						if (const FString* SceneImportAssetPathString = AssetUserData->MetaData.Find(Constants::SceneImportAssetPathKey))
						{
							if (const FString* FactoryNodeUid = AssetUserData->MetaData.Find(Constants::FactoryNodeUidPathKey))
							{
								if (UInterchangeSceneImportAsset* SceneImportAsset = GetSceneImportAssetFromPath(FSoftObjectPath(*SceneImportAssetPathString)))
								{
									if (SceneImportAsset->AssetImportData->GetStoredFactoryNode(*FactoryNodeUid) != nullptr)
									{
										return SceneImportAsset;
									}
								}
							}
						}
					}
				}
			}

			return nullptr;
		}

		const UInterchangeFactoryBaseNode* GetFactoryNodeForResetActor(const UInterchangeSceneImportAsset* SceneImportAsset, AActor* ActorToReset)
		{
			if (!SceneImportAsset || !SceneImportAsset->AssetImportData || !ActorToReset)
			{
				return nullptr;
			}

			for (UActorComponent* Component : ActorToReset->GetComponents())
			{
				if (UInterchangeAssetUserData* AssetUserData = Cast<UInterchangeAssetUserData>(Component->GetAssetUserDataOfClass(UInterchangeAssetUserData::StaticClass())))
				{
					using namespace UE::Interchange::InterchangeReset;
					if (const FString* FactoryNodeUid = AssetUserData->MetaData.Find(Constants::FactoryNodeUidPathKey))
					{
						return SceneImportAsset->AssetImportData->GetStoredFactoryNode(*FactoryNodeUid);
					}
				}
			}

			return SceneImportAsset->GetFactoryNodeFromPath(FSoftObjectPath(ActorToReset));
		}

		void SetWorldAssetForLevelInstanceActor(ALevelInstance* LevelInstanceActor, UWorld* ReferencedWorld)
		{
#if WITH_EDITORONLY_DATA
			UWorld* ParentWorld = LevelInstanceActor->GetWorld();

			// Make sure newly created level asset gets scanned
			ULevel::ScanLevelAssets(ReferencedWorld->GetPackage()->GetName());

			ParentWorld->PreEditChange(nullptr);

			LevelInstanceActor->SetWorldAsset(ReferencedWorld);
			LevelInstanceActor->UpdateLevelInstanceFromWorldAsset();
			LevelInstanceActor->LoadLevelInstance();

			//Reference world must be cleanup since they are not the main world.
			//This remove all the world managers and prevent GC issue when unloading the main world referencing this world.
			if (ReferencedWorld->bIsWorldInitialized)
			{
				ReferencedWorld->CleanupWorld();
			}

			ParentWorld->PostEditChange();
#endif
		}

		bool ExecuteLevelInstanceReset(const UInterchangeSceneImportAsset* SceneImportAsset, ALevelInstance* LevelInstanceActor)
		{
			if (!LevelInstanceActor || !SceneImportAsset)
			{
				return false;
			}

			ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(LevelInstanceActor);

			const TSoftObjectPtr<UWorld>& WorldObjectPtr = LevelInstanceActor->GetWorldAsset();
			FTransform CachedLevelInstanceActorTransform;

			if (const UWorld* World = Cast<UWorld>(WorldObjectPtr.ToSoftObjectPath().TryLoad()))
			{
				using namespace UE::Interchange::InterchangeReset;

				const FLevelInstanceID CurrentLevelInstanceID = LevelInstance->GetLevelInstanceID();
				const bool bWasPrevEditing = LevelInstance->IsEditing();
				if (!bWasPrevEditing && LevelInstance->CanEnterEdit())
				{
					CachedLevelInstanceActorTransform = LevelInstanceActor->GetActorTransform();
					LevelInstanceActor->SetActorTransform(FTransform::Identity);
					LevelInstance->EnterEdit(LevelInstanceActor);
				}

				FInterchangeResetParameters ResetParameters(SceneImportAsset);

				// Filter nodes
				if (CCvarInterchangeResetFilteredNodes->GetBool())
				{
					if (ULevel* LoadedLevel = LevelInstance->GetLoadedLevel())
					{
						TArray<AActor*> SceneActors = LoadedLevel->Actors;
						TSet<const UInterchangeFactoryBaseNode*> TempFilteredFactoryNodes;
						for (AActor* ChildActor : SceneActors)
						{
							if (const UInterchangeFactoryBaseNode* FactoryNode = GetFactoryNodeForResetActor(SceneImportAsset, ChildActor))
							{
								ResetParameters.AddObjectInstanceToReset(FactoryNode, ChildActor);
							}
						}
					}
				}

				FInterchangeReset::ExecuteReset(ResetParameters);

				if (!bWasPrevEditing && LevelInstance->CanExitEdit())
				{
					LevelInstance->ExitEdit();
					if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
					{
						if (ILevelInstanceInterface* LevelInstanceFromID = LevelInstanceSubsystem->GetLevelInstance(CurrentLevelInstanceID))
						{
							if (AActor* UpdatedLevelInstanceActor = Cast<AActor>(LevelInstanceFromID))
							{
								UpdatedLevelInstanceActor->SetActorTransform(CachedLevelInstanceActorTransform);
							}
						}
					}
				}

				return true;
			}

			return false;
		}

		void ExecuteWorldReset(UWorld* World, const UInterchangeSceneImportAsset* SceneImportAsset)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_Level_ContextMenuReset)
			if (!World || !SceneImportAsset)
			{
				return;
			}

			FWorldContext& EditorWorldContext = GEditor->GetEditorWorldContext();
			if (UWorld* EditorWorld = EditorWorldContext.World())
			{
				FActorSpawnParameters SpawnParameters;
				SpawnParameters.Name = FName(World->GetMapName());
				SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
				SpawnParameters.OverrideLevel = EditorWorld->PersistentLevel;
				SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

				if (ALevelInstance* SpawnedActor = EditorWorld->SpawnActor<ALevelInstance>(ALevelInstance::StaticClass(), SpawnParameters))
				{
					if (!SpawnedActor->GetRootComponent())
					{
						USceneComponent* RootComponent = NewObject<USceneComponent>(SpawnedActor, USceneComponent::GetDefaultSceneRootVariableName(), RF_Transactional);
#if WITH_EDITORONLY_DATA
						RootComponent->bVisualizeComponent = true;
#endif
						SpawnedActor->SetRootComponent(RootComponent);
						SpawnedActor->AddInstanceComponent(RootComponent);
					}

					SetWorldAssetForLevelInstanceActor(SpawnedActor, World);
					if (ExecuteLevelInstanceReset(SceneImportAsset, SpawnedActor))
					{
						GEditor->SelectNone(true, true);
					}

					SpawnedActor->Destroy();
				}
			}
		}
	}
}

void UInterchangeEditorScriptLibrary::ResetLevelAsset(UWorld* World)
{	
	using namespace UE::InterchangeEditorScriptLibrary::Private::InterchangeReset;

	if (!IsInterchangeResetAvailable())
	{
		return;
	}

	if (UInterchangeSceneImportAsset* SceneImportAsset = GetSceneImportAssetForWorld(World))
	{
		ExecuteWorldReset(World, SceneImportAsset);
	}
}

void UInterchangeEditorScriptLibrary::ResetSceneImportAsset(UInterchangeSceneImportAsset* SceneImportAsset)
{
	using namespace UE::InterchangeEditorScriptLibrary::Private::InterchangeReset;
	if (!IsInterchangeResetAvailable())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_SceneImportAsset_ContextMenuReset)
	FInterchangeResetParameters ResetParameters(SceneImportAsset);

	const FSoftObjectPath CurrentSceneImportObjectPath(SceneImportAsset);
	const TSubclassOf<UAssetUserData> AssetUserDataClass(UInterchangeLevelAssetUserData::StaticClass());

	ResetParameters.PreResetDelegates.OnNodeProcessed.BindLambda([SceneImportAsset, &CurrentSceneImportObjectPath, &AssetUserDataClass](FInterchangeResetParameters& ResetParameters, const UInterchangeFactoryBase* Factory, const UInterchangeFactoryBaseNode* FactoryNode)
	{
		if (const UInterchangeLevelFactoryNode* LevelFactoryNode = Cast<UInterchangeLevelFactoryNode>(FactoryNode))
		{
			bool bShouldCreateLevel = false;
			LevelFactoryNode->GetCustomShouldCreateLevel(bShouldCreateLevel);

			if (bShouldCreateLevel)
			{
				FSoftObjectPath WorldPath;
				if (LevelFactoryNode->GetCustomReferenceObject(WorldPath))
				{
					if (UWorld* ReferencedWorld = Cast<UWorld>(WorldPath.TryLoad()))
					{
						if (AWorldSettings* WorldSettings = ReferencedWorld->GetWorldSettings())
						{
							if (UInterchangeLevelAssetUserData* AssetUserData = Cast<UInterchangeLevelAssetUserData>(WorldSettings->GetAssetUserDataOfClass(AssetUserDataClass)))
							{
								using namespace UE::InterchangeEditorScriptLibrary::Private::InterchangeReset;

								bool bShouldCreateLevelInstance = false;
								for (const FString& SceneImportAssetPath : AssetUserData->SceneImportPaths)
								{
									if (FSoftObjectPath(SceneImportAssetPath) == CurrentSceneImportObjectPath)
									{
										bShouldCreateLevelInstance = true;
										break;
									}
								}

								if (!bShouldCreateLevelInstance)
								{
									return;
								}

								TObjectPtr<ALevelInstance> LevelInstance = SpawnLevelInstanceInEditor(FName(ReferencedWorld->GetName()));
								if (IsValid(LevelInstance))
								{
									InitializeLevelInstance(LevelInstance, ReferencedWorld);
									if (!ResetParameters.ResetContextData->ObjectsSpawnedDuringReset.Contains(FactoryNode))
									{
										ResetParameters.ResetContextData->ObjectsSpawnedDuringReset.Emplace(FactoryNode, TArray<TObjectPtr<UObject>>());
									}

									ResetParameters.ResetContextData->ObjectsSpawnedDuringReset[FactoryNode].Add(LevelInstance);

									if (LevelInstance->CanEnterEdit())
									{
										LevelInstance->EnterEdit();
									}
								}
							}
						}
					}
				}
			}
		}
	});

	ResetParameters.PostResetDelegates.OnNodeProcessed.BindLambda([SceneImportAsset, &CurrentSceneImportObjectPath, &AssetUserDataClass](FInterchangeResetParameters& ResetParameters, const UInterchangeFactoryBase* Factory, const UInterchangeFactoryBaseNode* FactoryNode)
		{
			if (const UInterchangeLevelFactoryNode* LevelFactoryNode = Cast<UInterchangeLevelFactoryNode>(FactoryNode))
			{
				bool bShouldCreateLevel = false;
				LevelFactoryNode->GetCustomShouldCreateLevel(bShouldCreateLevel);

				if (bShouldCreateLevel)
				{
					if (ResetParameters.ResetContextData->ObjectsSpawnedDuringReset.Contains(FactoryNode))
					{
						for (TObjectPtr<UObject>& SpawnedObject : ResetParameters.ResetContextData->ObjectsSpawnedDuringReset[FactoryNode])
						{
							if (ALevelInstance* LevelInstance = Cast<ALevelInstance>(SpawnedObject))
							{
								if (LevelInstance->IsEditing())
								{
#if WITH_EDITOR
									// Make sure transformation is up to date after registration as its possible LevelInstance actor can get unregistered when editing properties
									// through Details panel. In this case the ULevelInstanceComponent might not be able to update the ALevelInstanceEditorInstanceActor transform.
									LevelInstance->GetLevelInstanceComponent()->UpdateEditorInstanceActor();
#endif
									if (LevelInstance->CanExitEdit())
									{
										LevelInstance->ExitEdit();
										// Clear selection in case anything gets selected after applying the changes.
										GEditor->SelectNone(true, true);
									}
								}
								LevelInstance->Destroy();
							}
						}
					}
				}
			}
		});

	FInterchangeReset::ExecuteReset(ResetParameters);
	GEditor->RedrawAllViewports();
}

void UInterchangeEditorScriptLibrary::ResetActors(TArray<AActor*> Actors)
{
	using namespace UE::InterchangeEditorScriptLibrary::Private::InterchangeReset;

	if (!IsInterchangeResetAvailable())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(InterchangeEditorScriptLibrary::ResetActors)
	TMap<UInterchangeSceneImportAsset*, FInterchangeResetParameters> BatchedReset;
	TMap<UInterchangeSceneImportAsset*, ALevelInstance*> LevelInstanceResets;

	for (AActor* Actor : Actors)
	{
		if (!Actor)
		{
			continue;
		}

		if (UInterchangeSceneImportAsset* SceneImportAsset = GetSceneImportAsset(Actor))
		{
			if (ALevelInstance* LevelInstanceActor = Cast<ALevelInstance>(Actor))
			{
				if (!LevelInstanceResets.Contains(SceneImportAsset))
				{
					// It is enough to reset just one of the level instances as the changes would also be applied to all other when the reset is committed.
					LevelInstanceResets.Emplace(SceneImportAsset, LevelInstanceActor);
				}
			}
			else
			{
				if (!BatchedReset.Contains(SceneImportAsset))
				{
					BatchedReset.Emplace(SceneImportAsset, SceneImportAsset);
				}

				if (UE::Interchange::InterchangeReset::CCvarInterchangeResetFilteredNodes->GetBool())
				{
					if (const UInterchangeFactoryBaseNode* FactoryNode = GetFactoryNodeForResetActor(SceneImportAsset, Actor))
					{
						BatchedReset[SceneImportAsset].AddObjectInstanceToReset(FactoryNode, Actor);
					}
				}
			}
		}
	}

	for (TPair<UInterchangeSceneImportAsset*, FInterchangeResetParameters>& ResetDataPair : BatchedReset)
	{
		// As these are all actors in the scene, no need to do anything else to reset them.
		FInterchangeReset::ExecuteReset(ResetDataPair.Value);
	}

	for (TPair<UInterchangeSceneImportAsset*, ALevelInstance*>& ResetDataPair : LevelInstanceResets)
	{
		ExecuteLevelInstanceReset(ResetDataPair.Key, ResetDataPair.Value);;
	}

	GEditor->RedrawAllViewports();
}

bool UInterchangeEditorScriptLibrary::CanResetActor(const AActor* Actor)
{
	using namespace UE::InterchangeEditorScriptLibrary::Private::InterchangeReset;
	if (!IsInterchangeResetAvailable())
	{
		return false;
	}

	if (const ALevelInstance* LevelInstanceActor = Cast<ALevelInstance>(Actor))
	{
		const TSoftObjectPtr<UWorld>& WorldObjectPtr = LevelInstanceActor->GetWorldAsset();
		if (const UWorld* ReferencedWorld = Cast<UWorld>(WorldObjectPtr.ToSoftObjectPath().TryLoad()))
		{
			if (AWorldSettings* WorldSettings = ReferencedWorld->GetWorldSettings())
			{
				return WorldSettings->HasAssetUserDataOfClass(TSubclassOf<UAssetUserData>(UInterchangeLevelAssetUserData::StaticClass()));
			}
		}
	}
	else
	{
		TSubclassOf<UAssetUserData> InterchangeAssetUserDataClass(UInterchangeAssetUserData::StaticClass());
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component->HasAssetUserDataOfClass(InterchangeAssetUserDataClass))
			{
				return true;
			}
		}
	}

	return false;
}

bool UInterchangeEditorScriptLibrary::CanResetWorld(const UWorld* World)
{
	using namespace UE::InterchangeEditorScriptLibrary::Private::InterchangeReset;
	if (!IsInterchangeResetAvailable())
	{
		return false;
	}

	if (!World)
	{
		return false;
	}

	if (AWorldSettings* WorldSettings = World->GetWorldSettings())
	{
		return WorldSettings->HasAssetUserDataOfClass(TSubclassOf<UAssetUserData>(UInterchangeLevelAssetUserData::StaticClass()));
	}

	return false;
}

bool UInterchangeEditorScriptLibrary::LevelInstanceEnterEditMode(ALevelInstance* LevelInstance)
{
	if (!LevelInstance)
	{
		return false;
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(LevelInstance->GetWorld()))
	{
		if (LevelInstanceSubsystem->IsEditingLevelInstance(LevelInstance))
		{
			return true;
		}

		if (LevelInstanceSubsystem->CanEditLevelInstance(LevelInstance))
		{
			LevelInstanceSubsystem->EditLevelInstance(LevelInstance);
			return true;
		}
	}

	return false;
}

bool UInterchangeEditorScriptLibrary::LevelInstanceCommit(ALevelInstance* LevelInstance, bool bDiscardChanges)
{
	if (!LevelInstance)
	{
		return false;
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(LevelInstance->GetWorld()))
	{
		if (LevelInstanceSubsystem->CanCommitLevelInstance(LevelInstance))
		{
			LevelInstanceSubsystem->CommitLevelInstance(LevelInstance, bDiscardChanges);
			return true;
		}
	}

	return false;
}

const TArray<AActor*> UInterchangeEditorScriptLibrary::LevelInstanceGetEditableActors(ALevelInstance* LevelInstance)
{
	if (!LevelInstance)
	{
		return TArray<AActor*>();
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(LevelInstance->GetWorld()))
	{
		if (!LevelInstanceSubsystem->IsEditingLevelInstance(LevelInstance))
		{
			return TArray<AActor*>();
		}

		TSet<AActor*> EditorLevelActorsSet;
		TSet<AActor*> ReferencedLevelActorsSet;

		if (UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>())
		{
			EditorLevelActorsSet.Append(EditorActorSubsystem->GetAllLevelActors());
		}

		if (ULevel* LoadedLevel = LevelInstance->GetLoadedLevel())
		{
			ReferencedLevelActorsSet.Append(LoadedLevel->Actors);
		}

		return EditorLevelActorsSet.Intersect(ReferencedLevelActorsSet).Array();
	}

	return TArray<AActor*>();
}

TArray<AActor*> UInterchangeEditorScriptLibrary::LevelInstanceGetActors(ALevelInstance* LevelInstance)
{
	if (!LevelInstance)
	{
		return TArray<AActor*>();
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(LevelInstance->GetWorld()))
	{
		TArray<AActor*> Actors;
		LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstance, [&Actors](AActor* Actor)
		{
			if (Actor)
			{
				Actors.Add(Actor);
			}
			return true;
		});
		return Actors;
	}

	return TArray<AActor*>();
}

#undef LOCTEXT_NAMESPACE // "InterchangeResetEditorSubsystem"
