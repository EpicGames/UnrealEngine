// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeFbxAssetImportDataConverter.h"

#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshResources.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxAssetImportData.h"
#include "Factories/FbxImportUI.h"
#include "Factories/FbxMeshImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Fbx/InterchangeFbxTranslator.h"
#include "InterchangeAnimSequenceFactoryNode.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeGenericAnimationPipeline.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangeGenericAssetsPipelineSharedSettings.h"
#include "InterchangeGenericMaterialPipeline.h"
#include "InterchangeGenericMeshPipeline.h"
#include "InterchangeGenericTexturePipeline.h"
#include "InterchangeProjectSettings.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeFbxAssetImportDataConverter)

namespace UE::Interchange::Private
{
	//Create a generic asset pipeline, use the one from the project settings if its valid
	UInterchangeGenericAssetsPipeline* GetDefaultGenericAssetPipelineForConvertion(UObject* Outer)
	{
		//Create a generic asset pipeline, use the one from the project settings if its valid
		UInterchangeGenericAssetsPipeline* GenericAssetPipeline = nullptr;
		if (const UInterchangeProjectSettings* InterchangeProjectSettings = GetDefault<UInterchangeProjectSettings>())
		{
			if (UInterchangeGenericAssetsPipeline* ConvertDefaultPipelineAsset = Cast<UInterchangeGenericAssetsPipeline>(InterchangeProjectSettings->ConverterDefaultPipeline.TryLoad()))
			{
				GenericAssetPipeline = DuplicateObject<UInterchangeGenericAssetsPipeline>(ConvertDefaultPipelineAsset, Outer);
			}
		}

		if (!GenericAssetPipeline)
		{
			GenericAssetPipeline = NewObject<UInterchangeGenericAssetsPipeline>(Outer);
		}
		return GenericAssetPipeline;
	}

	void TransferSourceFileInformation(const UAssetImportData* SourceData, UAssetImportData* DestinationData)
	{
		TArray<FAssetImportInfo::FSourceFile> SourceFiles = SourceData->GetSourceData().SourceFiles;
		DestinationData->SetSourceFiles(MoveTemp(SourceFiles));
	}

	void FillFbxAssetImportData(const UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings, const UInterchangeGenericAssetsPipeline* GenericAssetPipeline, UFbxAssetImportData* AssetImportData)
	{
		if (InterchangeFbxTranslatorSettings)
		{
			AssetImportData->bConvertScene = InterchangeFbxTranslatorSettings->bConvertScene;
			AssetImportData->bConvertSceneUnit = InterchangeFbxTranslatorSettings->bConvertSceneUnit;
			AssetImportData->bForceFrontXAxis = InterchangeFbxTranslatorSettings->bForceFrontXAxis;
			
		}
		else if(UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettingsCDO = UInterchangeFbxTranslatorSettings::StaticClass()->GetDefaultObject<UInterchangeFbxTranslatorSettings>())
		{
			AssetImportData->bConvertScene = InterchangeFbxTranslatorSettingsCDO->bConvertScene;
			AssetImportData->bConvertSceneUnit = InterchangeFbxTranslatorSettingsCDO->bConvertSceneUnit;
			AssetImportData->bForceFrontXAxis = InterchangeFbxTranslatorSettingsCDO->bForceFrontXAxis;
		}
		else
		{
			AssetImportData->bConvertScene = true;
			AssetImportData->bConvertSceneUnit = true;
			AssetImportData->bForceFrontXAxis = false;
		}
		AssetImportData->bImportAsScene = false;
		AssetImportData->ImportRotation = GenericAssetPipeline->ImportOffsetRotation;
		AssetImportData->ImportTranslation = GenericAssetPipeline->ImportOffsetTranslation;
		AssetImportData->ImportUniformScale = GenericAssetPipeline->ImportOffsetUniformScale;
	}

	void FillFbxMeshImportData(const UInterchangeGenericAssetsPipeline* GenericAssetPipeline, UFbxMeshImportData* MeshImportData)
	{
		MeshImportData->bBakePivotInVertex = false;
		MeshImportData->bComputeWeightedNormals = GenericAssetPipeline->CommonMeshesProperties->bComputeWeightedNormals;
		MeshImportData->bImportMeshLODs = GenericAssetPipeline->CommonMeshesProperties->bImportLods;
		MeshImportData->bReorderMaterialToFbxOrder = true;
		MeshImportData->bTransformVertexToAbsolute = GenericAssetPipeline->CommonMeshesProperties->bBakeMeshes;
		MeshImportData->bBakePivotInVertex = GenericAssetPipeline->CommonMeshesProperties->bBakePivotMeshes;

		if (GenericAssetPipeline->CommonMeshesProperties->bUseMikkTSpace)
		{
			MeshImportData->NormalGenerationMethod = EFBXNormalGenerationMethod::MikkTSpace;
		}
		else
		{
			MeshImportData->NormalGenerationMethod = EFBXNormalGenerationMethod::BuiltIn;
		}

		if (GenericAssetPipeline->CommonMeshesProperties->bRecomputeNormals)
		{
			MeshImportData->NormalImportMethod = EFBXNormalImportMethod::FBXNIM_ComputeNormals;
		}
		else
		{
			if (GenericAssetPipeline->CommonMeshesProperties->bRecomputeTangents)
			{
				MeshImportData->NormalImportMethod = EFBXNormalImportMethod::FBXNIM_ImportNormals;
			}
			else
			{
				MeshImportData->NormalImportMethod = EFBXNormalImportMethod::FBXNIM_ImportNormalsAndTangents;
			}
		}
	}

	void FillFbxStaticMeshImportData(const UInterchangeGenericAssetsPipeline* GenericAssetPipeline, UFbxStaticMeshImportData* DestinationStaticMeshImportData)
	{
		DestinationStaticMeshImportData->bAutoGenerateCollision = GenericAssetPipeline->MeshPipeline->bCollision && GenericAssetPipeline->MeshPipeline->Collision != EInterchangeMeshCollision::None;
		DestinationStaticMeshImportData->bBuildNanite = GenericAssetPipeline->MeshPipeline->bBuildNanite;
		DestinationStaticMeshImportData->bBuildReversedIndexBuffer = GenericAssetPipeline->MeshPipeline->bBuildReversedIndexBuffer;
		DestinationStaticMeshImportData->bCombineMeshes = GenericAssetPipeline->MeshPipeline->CombineStaticMeshesBehavior == EInterchangeCombineStaticMeshesBehavior::All;
		DestinationStaticMeshImportData->bGenerateLightmapUVs = GenericAssetPipeline->MeshPipeline->bGenerateLightmapUVs;
		DestinationStaticMeshImportData->bOneConvexHullPerUCX = GenericAssetPipeline->MeshPipeline->bOneConvexHullPerUCX;
		DestinationStaticMeshImportData->bRemoveDegenerates = GenericAssetPipeline->CommonMeshesProperties->bRemoveDegenerates;
		DestinationStaticMeshImportData->DistanceFieldResolutionScale = GenericAssetPipeline->MeshPipeline->DistanceFieldResolutionScale;
		DestinationStaticMeshImportData->StaticMeshLODGroup = GenericAssetPipeline->MeshPipeline->LodGroup;
		if (GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption == EInterchangeVertexColorImportOption::IVCIO_Ignore)
		{
			DestinationStaticMeshImportData->VertexColorImportOption = EVertexColorImportOption::Ignore;
		}
		else if (GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption == EInterchangeVertexColorImportOption::IVCIO_Override)
		{
			DestinationStaticMeshImportData->VertexColorImportOption = EVertexColorImportOption::Override;
		}
		else if (GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption == EInterchangeVertexColorImportOption::IVCIO_Replace)
		{
			DestinationStaticMeshImportData->VertexColorImportOption = EVertexColorImportOption::Replace;
		}
		DestinationStaticMeshImportData->VertexOverrideColor = GenericAssetPipeline->CommonMeshesProperties->VertexOverrideColor;
	}

	void FillFbxSkeletalMeshImportData(const UInterchangeGenericAssetsPipeline* GenericAssetPipeline, UFbxSkeletalMeshImportData* DestinationSkeletalMeshImportData)
	{
		DestinationSkeletalMeshImportData->bImportMeshesInBoneHierarchy = GenericAssetPipeline->CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals;
		DestinationSkeletalMeshImportData->bImportMorphTargets = GenericAssetPipeline->MeshPipeline->bImportMorphTargets;
		DestinationSkeletalMeshImportData->bImportVertexAttributes = GenericAssetPipeline->MeshPipeline->bImportVertexAttributes;
		DestinationSkeletalMeshImportData->bKeepSectionsSeparate = GenericAssetPipeline->CommonMeshesProperties->bKeepSectionsSeparate;
		DestinationSkeletalMeshImportData->bPreserveSmoothingGroups = true;
		DestinationSkeletalMeshImportData->bUpdateSkeletonReferencePose = GenericAssetPipeline->MeshPipeline->bUpdateSkeletonReferencePose;
		DestinationSkeletalMeshImportData->bUseT0AsRefPose = GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->bUseT0AsRefPose;
		if (GenericAssetPipeline->MeshPipeline->SkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::All)
		{
			DestinationSkeletalMeshImportData->ImportContentType = EFBXImportContentType::FBXICT_All;
		}
		else if (GenericAssetPipeline->MeshPipeline->SkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::Geometry)
		{
			DestinationSkeletalMeshImportData->ImportContentType = EFBXImportContentType::FBXICT_Geometry;
		}
		else if (GenericAssetPipeline->MeshPipeline->SkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::SkinningWeights)
		{
			DestinationSkeletalMeshImportData->ImportContentType = EFBXImportContentType::FBXICT_SkinningWeights;
		}

		if (GenericAssetPipeline->MeshPipeline->LastSkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::All)
		{
			DestinationSkeletalMeshImportData->LastImportContentType = EFBXImportContentType::FBXICT_All;
		}
		else if (GenericAssetPipeline->MeshPipeline->LastSkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::Geometry)
		{
			DestinationSkeletalMeshImportData->LastImportContentType = EFBXImportContentType::FBXICT_Geometry;
		}
		else if (GenericAssetPipeline->MeshPipeline->LastSkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::SkinningWeights)
		{
			DestinationSkeletalMeshImportData->LastImportContentType = EFBXImportContentType::FBXICT_SkinningWeights;
		}

		DestinationSkeletalMeshImportData->MorphThresholdPosition = GenericAssetPipeline->MeshPipeline->MorphThresholdPosition;
		DestinationSkeletalMeshImportData->ThresholdPosition = GenericAssetPipeline->MeshPipeline->ThresholdPosition;
		DestinationSkeletalMeshImportData->ThresholdTangentNormal = GenericAssetPipeline->MeshPipeline->ThresholdTangentNormal;
		DestinationSkeletalMeshImportData->ThresholdUV = GenericAssetPipeline->MeshPipeline->ThresholdUV;

		if (GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption == EInterchangeVertexColorImportOption::IVCIO_Ignore)
		{
			DestinationSkeletalMeshImportData->VertexColorImportOption = EVertexColorImportOption::Ignore;
		}
		else if (GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption == EInterchangeVertexColorImportOption::IVCIO_Override)
		{
			DestinationSkeletalMeshImportData->VertexColorImportOption = EVertexColorImportOption::Override;
		}
		else if (GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption == EInterchangeVertexColorImportOption::IVCIO_Replace)
		{
			DestinationSkeletalMeshImportData->VertexColorImportOption = EVertexColorImportOption::Replace;
		}
		DestinationSkeletalMeshImportData->VertexOverrideColor = GenericAssetPipeline->CommonMeshesProperties->VertexOverrideColor;
	}

	void FillFbxAnimSequenceImportData(const UInterchangeGenericAssetsPipeline* GenericAssetPipeline, UFbxAnimSequenceImportData* DestinationAnimSequenceImportData)
	{
		switch (GenericAssetPipeline->AnimationPipeline->AnimationRange)
		{
		case EInterchangeAnimationRange::Timeline:
		{
			DestinationAnimSequenceImportData->AnimationLength = EFBXAnimationLengthImportType::FBXALIT_ExportedTime;
			break;
		}
		case EInterchangeAnimationRange::Animated:
		{
			DestinationAnimSequenceImportData->AnimationLength = EFBXAnimationLengthImportType::FBXALIT_AnimatedKey;
			break;
		}
		case EInterchangeAnimationRange::SetRange:
		{
			DestinationAnimSequenceImportData->AnimationLength = EFBXAnimationLengthImportType::FBXALIT_SetRange;
			break;
		}
		}
		DestinationAnimSequenceImportData->bAddCurveMetadataToSkeleton = GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->bAddCurveMetadataToSkeleton;
		DestinationAnimSequenceImportData->bDeleteExistingCustomAttributeCurves = GenericAssetPipeline->AnimationPipeline->bDeleteExistingCustomAttributeCurves;
		DestinationAnimSequenceImportData->bDeleteExistingMorphTargetCurves = GenericAssetPipeline->AnimationPipeline->bDeleteExistingMorphTargetCurves;
		DestinationAnimSequenceImportData->bDeleteExistingNonCurveCustomAttributes = GenericAssetPipeline->AnimationPipeline->bDeleteExistingNonCurveCustomAttributes;
		DestinationAnimSequenceImportData->bDoNotImportCurveWithZero = GenericAssetPipeline->AnimationPipeline->bDoNotImportCurveWithZero;
		DestinationAnimSequenceImportData->bImportBoneTracks = GenericAssetPipeline->AnimationPipeline->bImportBoneTracks;
		DestinationAnimSequenceImportData->bImportCustomAttribute = GenericAssetPipeline->AnimationPipeline->bImportCustomAttribute;
		DestinationAnimSequenceImportData->bImportMeshesInBoneHierarchy = GenericAssetPipeline->CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals;
		DestinationAnimSequenceImportData->bPreserveLocalTransform = false;
		DestinationAnimSequenceImportData->bRemoveRedundantKeys = GenericAssetPipeline->AnimationPipeline->bRemoveCurveRedundantKeys;
		DestinationAnimSequenceImportData->bSetMaterialDriveParameterOnCustomAttribute = GenericAssetPipeline->AnimationPipeline->bSetMaterialDriveParameterOnCustomAttribute;
		DestinationAnimSequenceImportData->bSnapToClosestFrameBoundary = GenericAssetPipeline->AnimationPipeline->FrameAlignment == EInterchangeFrameAlignment::SnapToClosest ? true : false;
		DestinationAnimSequenceImportData->bUseDefaultSampleRate = GenericAssetPipeline->AnimationPipeline->bUse30HzToBakeBoneAnimation;
		DestinationAnimSequenceImportData->CustomSampleRate = GenericAssetPipeline->AnimationPipeline->CustomBoneAnimationSampleRate;
		DestinationAnimSequenceImportData->FrameImportRange = GenericAssetPipeline->AnimationPipeline->FrameImportRange;
		DestinationAnimSequenceImportData->MaterialCurveSuffixes = GenericAssetPipeline->AnimationPipeline->MaterialCurveSuffixes;
		DestinationAnimSequenceImportData->SourceAnimationName = GenericAssetPipeline->AnimationPipeline->SourceAnimationName;
	}

	void FillInterchangeGenericAssetsPipelineFromFbxMeshImportData(UInterchangeGenericAssetsPipeline* GenericAssetPipeline, const UFbxMeshImportData* LegacyMeshImportData)
	{
		if (!LegacyMeshImportData || !GenericAssetPipeline)
		{
			return;
		}

		GenericAssetPipeline->CommonMeshesProperties->bComputeWeightedNormals = LegacyMeshImportData->bComputeWeightedNormals;
		GenericAssetPipeline->CommonMeshesProperties->bImportLods = LegacyMeshImportData->bImportMeshLODs;
		GenericAssetPipeline->CommonMeshesProperties->bBakeMeshes = LegacyMeshImportData->bTransformVertexToAbsolute;
		GenericAssetPipeline->CommonMeshesProperties->bBakePivotMeshes = LegacyMeshImportData->bBakePivotInVertex;

		if (LegacyMeshImportData->NormalGenerationMethod == EFBXNormalGenerationMethod::MikkTSpace)
		{
			GenericAssetPipeline->CommonMeshesProperties->bUseMikkTSpace = true;
		}
		else
		{
			GenericAssetPipeline->CommonMeshesProperties->bUseMikkTSpace = false;
		}

		if (LegacyMeshImportData->NormalImportMethod == EFBXNormalImportMethod::FBXNIM_ImportNormalsAndTangents)
		{
			GenericAssetPipeline->CommonMeshesProperties->bRecomputeNormals = false;
			GenericAssetPipeline->CommonMeshesProperties->bRecomputeTangents = false;
		}
		else if (LegacyMeshImportData->NormalImportMethod == EFBXNormalImportMethod::FBXNIM_ImportNormals)
		{
			GenericAssetPipeline->CommonMeshesProperties->bRecomputeNormals = false;
			GenericAssetPipeline->CommonMeshesProperties->bRecomputeTangents = true;
		}
		else if (LegacyMeshImportData->NormalImportMethod == EFBXNormalImportMethod::FBXNIM_ComputeNormals)
		{
			GenericAssetPipeline->CommonMeshesProperties->bRecomputeNormals = true;
			GenericAssetPipeline->CommonMeshesProperties->bRecomputeTangents = true;
		}
	}

	void FillInterchangeGenericAssetsPipelineFromFbxStaticMesh(UInterchangeGenericAssetsPipeline* GenericAssetPipeline, const UStaticMesh* StaticMesh)
	{
		if (!StaticMesh || !GenericAssetPipeline)
		{
			return;
		}

		GenericAssetPipeline->MeshPipeline->bAutoComputeLODScreenSizes = StaticMesh->GetAutoComputeLODScreenSize();
		if (const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData())
		{
			GenericAssetPipeline->MeshPipeline->LODScreenSizes.Empty();
			for (int32 LodIndex = 0; LodIndex < MAX_STATIC_MESH_LODS; ++LodIndex)
			{
				GenericAssetPipeline->MeshPipeline->LODScreenSizes.Add(RenderData->ScreenSize[LodIndex].Default);
			}
		}
	}

	void FillInterchangeGenericAssetsPipelineFromFbxStaticMeshImportData(UInterchangeGenericAssetsPipeline* GenericAssetPipeline
		, const UFbxStaticMeshImportData* StaticMeshImportData
		, bool bFillBaseClass = true)
	{
		if (!StaticMeshImportData || !GenericAssetPipeline)
		{
			return;
		}

		if (bFillBaseClass)
		{
			FillInterchangeGenericAssetsPipelineFromFbxMeshImportData(GenericAssetPipeline, StaticMeshImportData);
		}

		GenericAssetPipeline->MeshPipeline->bCollision = StaticMeshImportData->bAutoGenerateCollision;
		GenericAssetPipeline->MeshPipeline->Collision = StaticMeshImportData->bAutoGenerateCollision ? EInterchangeMeshCollision::Convex18DOP : EInterchangeMeshCollision::None;
		GenericAssetPipeline->MeshPipeline->bBuildNanite = StaticMeshImportData->bBuildNanite;
		GenericAssetPipeline->MeshPipeline->bBuildReversedIndexBuffer = StaticMeshImportData->bBuildReversedIndexBuffer;
		GenericAssetPipeline->MeshPipeline->CombineStaticMeshesBehavior = StaticMeshImportData->bCombineMeshes ? EInterchangeCombineStaticMeshesBehavior::All : EInterchangeCombineStaticMeshesBehavior::DoNotCombine;
		GenericAssetPipeline->MeshPipeline->bGenerateLightmapUVs = StaticMeshImportData->bGenerateLightmapUVs;
		GenericAssetPipeline->MeshPipeline->bOneConvexHullPerUCX = StaticMeshImportData->bOneConvexHullPerUCX;
		GenericAssetPipeline->CommonMeshesProperties->bRemoveDegenerates = StaticMeshImportData->bRemoveDegenerates;
		GenericAssetPipeline->MeshPipeline->DistanceFieldResolutionScale = StaticMeshImportData->DistanceFieldResolutionScale;
		GenericAssetPipeline->MeshPipeline->LodGroup = StaticMeshImportData->StaticMeshLODGroup;
		if (StaticMeshImportData->VertexColorImportOption == EVertexColorImportOption::Ignore)
		{
			GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Ignore;
		}
		else if (StaticMeshImportData->VertexColorImportOption == EVertexColorImportOption::Override)
		{
			GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Override;
		}
		else if (StaticMeshImportData->VertexColorImportOption == EVertexColorImportOption::Replace)
		{
			GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Replace;
		}

		GenericAssetPipeline->CommonMeshesProperties->VertexOverrideColor = StaticMeshImportData->VertexOverrideColor;
	}

	void FillInterchangeGenericAssetsPipelineFromFbxSkeletalMeshImportData(UInterchangeGenericAssetsPipeline* GenericAssetPipeline
		, const UFbxSkeletalMeshImportData* SkeletalMeshImportData
		, bool bFillBaseClass = true)
	{
		if (!SkeletalMeshImportData || !GenericAssetPipeline)
		{
			return;
		}

		if (bFillBaseClass)
		{
			FillInterchangeGenericAssetsPipelineFromFbxMeshImportData(GenericAssetPipeline, SkeletalMeshImportData);
		}

		GenericAssetPipeline->CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals = SkeletalMeshImportData->bImportMeshesInBoneHierarchy;
		GenericAssetPipeline->CommonMeshesProperties->bKeepSectionsSeparate = SkeletalMeshImportData->bKeepSectionsSeparate;
		GenericAssetPipeline->MeshPipeline->bCreatePhysicsAsset = false;
		GenericAssetPipeline->MeshPipeline->bImportMorphTargets = SkeletalMeshImportData->bImportMorphTargets;
		GenericAssetPipeline->MeshPipeline->bImportVertexAttributes = SkeletalMeshImportData->bImportVertexAttributes;
		GenericAssetPipeline->MeshPipeline->bUpdateSkeletonReferencePose = SkeletalMeshImportData->bUpdateSkeletonReferencePose;
		GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->bUseT0AsRefPose = SkeletalMeshImportData->bUseT0AsRefPose;
		if (SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_All)
		{
			GenericAssetPipeline->MeshPipeline->SkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::All;
		}
		else if (SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_Geometry)
		{
			GenericAssetPipeline->MeshPipeline->SkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::Geometry;
		}
		else if (SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_SkinningWeights)
		{
			GenericAssetPipeline->MeshPipeline->SkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::SkinningWeights;
		}

		if (SkeletalMeshImportData->LastImportContentType == EFBXImportContentType::FBXICT_All)
		{
			GenericAssetPipeline->MeshPipeline->LastSkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::All;
		}
		else if (SkeletalMeshImportData->LastImportContentType == EFBXImportContentType::FBXICT_Geometry)
		{
			GenericAssetPipeline->MeshPipeline->LastSkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::Geometry;
		}
		else if (SkeletalMeshImportData->LastImportContentType == EFBXImportContentType::FBXICT_SkinningWeights)
		{
			GenericAssetPipeline->MeshPipeline->LastSkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::SkinningWeights;
		}

		GenericAssetPipeline->MeshPipeline->MorphThresholdPosition = SkeletalMeshImportData->MorphThresholdPosition;
		GenericAssetPipeline->MeshPipeline->ThresholdPosition = SkeletalMeshImportData->ThresholdPosition;
		GenericAssetPipeline->MeshPipeline->ThresholdTangentNormal = SkeletalMeshImportData->ThresholdTangentNormal;
		GenericAssetPipeline->MeshPipeline->ThresholdUV = SkeletalMeshImportData->ThresholdUV;

		if (SkeletalMeshImportData->VertexColorImportOption == EVertexColorImportOption::Ignore)
		{
			GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Ignore;
		}
		else if (SkeletalMeshImportData->VertexColorImportOption == EVertexColorImportOption::Override)
		{
			GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Override;
		}
		else if (SkeletalMeshImportData->VertexColorImportOption == EVertexColorImportOption::Replace)
		{
			GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Replace;
		}
		GenericAssetPipeline->CommonMeshesProperties->VertexOverrideColor = SkeletalMeshImportData->VertexOverrideColor;
	}

	void FillInterchangeGenericAssetsPipelineFromFbxAnimSequenceImportData(UInterchangeGenericAssetsPipeline* GenericAssetPipeline
		, const UFbxAnimSequenceImportData* AnimSequenceImportData)
	{
		if (!AnimSequenceImportData || !GenericAssetPipeline)
		{
			return;
		}

		switch (AnimSequenceImportData->AnimationLength)
		{
		case EFBXAnimationLengthImportType::FBXALIT_ExportedTime:
		{
			GenericAssetPipeline->AnimationPipeline->AnimationRange = EInterchangeAnimationRange::Timeline;
			break;
		}
		case EFBXAnimationLengthImportType::FBXALIT_AnimatedKey:
		{
			GenericAssetPipeline->AnimationPipeline->AnimationRange = EInterchangeAnimationRange::Animated;
			break;
		}
		case EFBXAnimationLengthImportType::FBXALIT_SetRange:
		{
			GenericAssetPipeline->AnimationPipeline->AnimationRange = EInterchangeAnimationRange::SetRange;
			break;
		}
		}
		GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->bAddCurveMetadataToSkeleton = AnimSequenceImportData->bAddCurveMetadataToSkeleton;
		GenericAssetPipeline->AnimationPipeline->bDeleteExistingCustomAttributeCurves = AnimSequenceImportData->bDeleteExistingCustomAttributeCurves;
		GenericAssetPipeline->AnimationPipeline->bDeleteExistingMorphTargetCurves = AnimSequenceImportData->bDeleteExistingMorphTargetCurves;
		GenericAssetPipeline->AnimationPipeline->bDeleteExistingNonCurveCustomAttributes = AnimSequenceImportData->bDeleteExistingNonCurveCustomAttributes;
		GenericAssetPipeline->AnimationPipeline->bDoNotImportCurveWithZero = AnimSequenceImportData->bDoNotImportCurveWithZero;
		GenericAssetPipeline->AnimationPipeline->bImportBoneTracks = AnimSequenceImportData->bImportBoneTracks;
		GenericAssetPipeline->AnimationPipeline->bImportCustomAttribute = AnimSequenceImportData->bImportCustomAttribute;
		GenericAssetPipeline->CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals = AnimSequenceImportData->bImportMeshesInBoneHierarchy;
		GenericAssetPipeline->AnimationPipeline->bRemoveCurveRedundantKeys = AnimSequenceImportData->bRemoveRedundantKeys;
		GenericAssetPipeline->AnimationPipeline->bSetMaterialDriveParameterOnCustomAttribute = AnimSequenceImportData->bSetMaterialDriveParameterOnCustomAttribute;
		GenericAssetPipeline->AnimationPipeline->FrameAlignment = AnimSequenceImportData->bSnapToClosestFrameBoundary ? EInterchangeFrameAlignment::SnapToClosest : EInterchangeFrameAlignment::None;
		GenericAssetPipeline->AnimationPipeline->bUse30HzToBakeBoneAnimation = AnimSequenceImportData->bUseDefaultSampleRate;
		GenericAssetPipeline->AnimationPipeline->CustomBoneAnimationSampleRate = AnimSequenceImportData->CustomSampleRate;
		GenericAssetPipeline->AnimationPipeline->FrameImportRange = AnimSequenceImportData->FrameImportRange;
		GenericAssetPipeline->AnimationPipeline->MaterialCurveSuffixes = AnimSequenceImportData->MaterialCurveSuffixes;
		GenericAssetPipeline->AnimationPipeline->SourceAnimationName = AnimSequenceImportData->SourceAnimationName;
	}

	UAssetImportData* ConvertToLegacyFbx(UStaticMesh* StaticMesh, const UInterchangeAssetImportData* InterchangeSourceData)
	{
		if (!InterchangeSourceData || !StaticMesh)
		{
			return nullptr;
		}

		//Create a fbx asset import data and fill the options
		UFbxStaticMeshImportData* DestinationStaticMeshImportData = NewObject<UFbxStaticMeshImportData>(StaticMesh);

		if (!DestinationStaticMeshImportData)
		{
			return nullptr;
		}

		//Transfer the Source file information
		TransferSourceFileInformation(InterchangeSourceData, DestinationStaticMeshImportData);

		const UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings = Cast<UInterchangeFbxTranslatorSettings>(InterchangeSourceData->GetTranslatorSettings());

		//Now find the generic asset pipeline
		for (UObject* Pipeline : InterchangeSourceData->GetPipelines())
		{
			if (const UInterchangeGenericAssetsPipeline* GenericAssetPipeline = Cast<UInterchangeGenericAssetsPipeline>(Pipeline))
			{

				FillFbxAssetImportData(InterchangeFbxTranslatorSettings, GenericAssetPipeline, DestinationStaticMeshImportData);
				FillFbxMeshImportData(GenericAssetPipeline, DestinationStaticMeshImportData);
				FillFbxStaticMeshImportData(GenericAssetPipeline, DestinationStaticMeshImportData);
				//Fill the reimport material match data and section data
				FImportMeshLodSectionsData SectionData;
				for (const FStaticMaterial& Material : StaticMesh->GetStaticMaterials())
				{
					DestinationStaticMeshImportData->ImportMaterialOriginalNameData.Add(Material.ImportedMaterialSlotName);
					SectionData.SectionOriginalMaterialName.Add(Material.ImportedMaterialSlotName);
				}
				DestinationStaticMeshImportData->ImportMeshLodData.Add(SectionData);
			}
		}
		return DestinationStaticMeshImportData;
	}

	UAssetImportData* ConvertToLegacyFbx(USkeletalMesh* SkeletalMesh, const UInterchangeAssetImportData* InterchangeSourceData)
	{
		if (!InterchangeSourceData || !SkeletalMesh)
		{
			return nullptr;
		}

		//Create a fbx asset import data and fill the options
		UFbxSkeletalMeshImportData* DestinationSkeletalMeshImportData = NewObject<UFbxSkeletalMeshImportData>(SkeletalMesh);

		if (!DestinationSkeletalMeshImportData)
		{
			return nullptr;
		}

		//Transfer the Source file information
		TransferSourceFileInformation(InterchangeSourceData, DestinationSkeletalMeshImportData);

		const UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings = Cast<UInterchangeFbxTranslatorSettings>(InterchangeSourceData->GetTranslatorSettings());

		//Now find the generic asset pipeline
		for (UObject* Pipeline : InterchangeSourceData->GetPipelines())
		{
			if (const UInterchangeGenericAssetsPipeline* GenericAssetPipeline = Cast<UInterchangeGenericAssetsPipeline>(Pipeline))
			{
				FillFbxAssetImportData(InterchangeFbxTranslatorSettings, GenericAssetPipeline, DestinationSkeletalMeshImportData);
				FillFbxMeshImportData(GenericAssetPipeline, DestinationSkeletalMeshImportData);
				FillFbxSkeletalMeshImportData(GenericAssetPipeline, DestinationSkeletalMeshImportData);
				//Fill the reimport material match data and section data
				FImportMeshLodSectionsData SectionData;
				for (const FSkeletalMaterial& Material : SkeletalMesh->GetMaterials())
				{
					DestinationSkeletalMeshImportData->ImportMaterialOriginalNameData.Add(Material.ImportedMaterialSlotName);
					SectionData.SectionOriginalMaterialName.Add(Material.ImportedMaterialSlotName);
				}
				DestinationSkeletalMeshImportData->ImportMeshLodData.Add(SectionData);
			}
		}
		return DestinationSkeletalMeshImportData;
	}

	UAssetImportData* ConvertToLegacyFbx(UAnimSequence* AnimSequence, const UInterchangeAssetImportData* InterchangeSourceData)
	{
		if (!InterchangeSourceData || !AnimSequence)
		{
			return nullptr;
		}

		//Create a fbx asset import data and fill the options
		UFbxAnimSequenceImportData* DestinationAnimSequenceImportData = NewObject<UFbxAnimSequenceImportData>(AnimSequence);

		if (!DestinationAnimSequenceImportData)
		{
			return nullptr;
		}

		//Transfer the Source file information
		TransferSourceFileInformation(InterchangeSourceData, DestinationAnimSequenceImportData);

		const UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings = Cast<UInterchangeFbxTranslatorSettings>(InterchangeSourceData->GetTranslatorSettings());

		//Now find the generic asset pipeline
		for (UObject* Pipeline : InterchangeSourceData->GetPipelines())
		{
			if (const UInterchangeGenericAssetsPipeline* GenericAssetPipeline = Cast<UInterchangeGenericAssetsPipeline>(Pipeline))
			{
				FillFbxAssetImportData(InterchangeFbxTranslatorSettings, GenericAssetPipeline, DestinationAnimSequenceImportData);
				FillFbxAnimSequenceImportData(GenericAssetPipeline, DestinationAnimSequenceImportData);
			}
		}
		return DestinationAnimSequenceImportData;
	}

	UFbxImportUI* ConvertToLegacyFbx(UObject* Owner, const UInterchangeAssetImportData* InterchangeSourceData)
	{
		if (!InterchangeSourceData || !Owner)
		{
			return nullptr;
		}
		//Create a fbx asset import data and fill the options
		UFbxImportUI* DestinationData = NewObject<UFbxImportUI>(Owner);

		if (!DestinationData)
		{
			return nullptr;
		}

		const UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings = Cast<UInterchangeFbxTranslatorSettings>(InterchangeSourceData->GetTranslatorSettings());
		const UInterchangeGenericAssetsPipeline* GenericAssetPipeline = nullptr;
		//Now find the generic asset pipeline
		for (UObject* Pipeline : InterchangeSourceData->GetPipelines())
		{
			if (const UInterchangeGenericAssetsPipeline* AssetPipeline = Cast<UInterchangeGenericAssetsPipeline>(Pipeline))
			{
				GenericAssetPipeline = AssetPipeline;
			}
		}

		if (!GenericAssetPipeline)
		{
			//Since we did not find any generic asset pipeline we fallback on the generic pipeline from the project settings conversion
			GenericAssetPipeline = GetDefaultGenericAssetPipelineForConvertion(GetTransientPackage());
		}

		FillFbxAssetImportData(InterchangeFbxTranslatorSettings, GenericAssetPipeline, DestinationData->StaticMeshImportData);
		FillFbxAssetImportData(InterchangeFbxTranslatorSettings, GenericAssetPipeline, DestinationData->SkeletalMeshImportData);
		FillFbxAssetImportData(InterchangeFbxTranslatorSettings, GenericAssetPipeline, DestinationData->AnimSequenceImportData);
		FillFbxAssetImportData(InterchangeFbxTranslatorSettings, GenericAssetPipeline, DestinationData->TextureImportData);

		FillFbxMeshImportData(GenericAssetPipeline, DestinationData->StaticMeshImportData);
		FillFbxMeshImportData(GenericAssetPipeline, DestinationData->SkeletalMeshImportData);

		FillFbxStaticMeshImportData(GenericAssetPipeline, DestinationData->StaticMeshImportData);
		FillFbxSkeletalMeshImportData(GenericAssetPipeline, DestinationData->SkeletalMeshImportData);

		FillFbxAnimSequenceImportData(GenericAssetPipeline, DestinationData->AnimSequenceImportData);

		DestinationData->bOverrideFullName = GenericAssetPipeline->bUseSourceNameForAsset;
		
		// LOD Screen Sizes
		{
			UInterchangeGenericMeshPipeline* MeshPipeline = GenericAssetPipeline->MeshPipeline;
			DestinationData->bAutoComputeLodDistances = MeshPipeline->bAutoComputeLODScreenSizes;
			DestinationData->LodDistance0 = MeshPipeline->LODScreenSizes.IsValidIndex(0) ? MeshPipeline->LODScreenSizes[0] : 0.0f;
			DestinationData->LodDistance1 = MeshPipeline->LODScreenSizes.IsValidIndex(1) ? MeshPipeline->LODScreenSizes[1] : 0.0f;
			DestinationData->LodDistance2 = MeshPipeline->LODScreenSizes.IsValidIndex(2) ? MeshPipeline->LODScreenSizes[2] : 0.0f;
			DestinationData->LodDistance3 = MeshPipeline->LODScreenSizes.IsValidIndex(3) ? MeshPipeline->LODScreenSizes[3] : 0.0f;
			DestinationData->LodDistance4 = MeshPipeline->LODScreenSizes.IsValidIndex(4) ? MeshPipeline->LODScreenSizes[4] : 0.0f;
			DestinationData->LodDistance5 = MeshPipeline->LODScreenSizes.IsValidIndex(5) ? MeshPipeline->LODScreenSizes[5] : 0.0f;
			DestinationData->LodDistance6 = MeshPipeline->LODScreenSizes.IsValidIndex(6) ? MeshPipeline->LODScreenSizes[6] : 0.0f;
			DestinationData->LodDistance7 = MeshPipeline->LODScreenSizes.IsValidIndex(7) ? MeshPipeline->LODScreenSizes[7] : 0.0f;
		}

		//Material Options
		DestinationData->bImportMaterials = GenericAssetPipeline->MaterialPipeline->bImportMaterials;
		if (!GenericAssetPipeline->MaterialPipeline->bReuseExistingMaterials)
		{
			DestinationData->TextureImportData->MaterialSearchLocation = EMaterialSearchLocation::DoNotSearch;
		}
		else
		{
			switch (GenericAssetPipeline->MaterialPipeline->SearchLocation)
			{
			case EInterchangeMaterialSearchLocation::Local:
				DestinationData->TextureImportData->MaterialSearchLocation = EMaterialSearchLocation::Local;
				break;
			case EInterchangeMaterialSearchLocation::UnderParent:
				DestinationData->TextureImportData->MaterialSearchLocation = EMaterialSearchLocation::UnderParent;
				break;
			case EInterchangeMaterialSearchLocation::UnderRoot:
				DestinationData->TextureImportData->MaterialSearchLocation = EMaterialSearchLocation::UnderRoot;
				break;
			case EInterchangeMaterialSearchLocation::AllAssets:
				DestinationData->TextureImportData->MaterialSearchLocation = EMaterialSearchLocation::AllAssets;
				break;
			}
		}

		if (GenericAssetPipeline->MaterialPipeline->ParentMaterial.IsAsset())
		{
			DestinationData->TextureImportData->bUseBaseMaterial = true;
			DestinationData->TextureImportData->BaseMaterialName = GenericAssetPipeline->MaterialPipeline->ParentMaterial;
		}
		else
		{
			DestinationData->TextureImportData->bUseBaseMaterial = false;
			DestinationData->TextureImportData->BaseMaterialName.Reset();
		}

		//Texture Options
		DestinationData->bImportTextures = GenericAssetPipeline->MaterialPipeline->TexturePipeline->bImportTextures;
		DestinationData->TextureImportData->bInvertNormalMaps = GenericAssetPipeline->MaterialPipeline->TexturePipeline->bFlipNormalMapGreenChannel;

		//Discover if we must import something in particular
		if (GenericAssetPipeline->MeshPipeline->CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_SkeletalMesh)
		{
			DestinationData->MeshTypeToImport = EFBXImportType::FBXIT_SkeletalMesh;
			DestinationData->bImportAsSkeletal = true;
			DestinationData->bImportAnimations = GenericAssetPipeline->AnimationPipeline->bImportAnimations;
		}
		else if (GenericAssetPipeline->MeshPipeline->CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_StaticMesh)
		{
			DestinationData->MeshTypeToImport = EFBXImportType::FBXIT_StaticMesh;

			DestinationData->bImportAsSkeletal = false;
			DestinationData->bImportAnimations = false;
		}
		else
		{
			DestinationData->bAutomatedImportShouldDetectType = true;
		}
		
		if (GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
		{
			DestinationData->Skeleton = GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->Skeleton.LoadSynchronous();
		}
		
		return DestinationData;
	}

	UAssetImportData* ConvertToInterchange(UObject* Obj, const UFbxAssetImportData* FbxAssetImportData)
	{
		if (!FbxAssetImportData || !Obj)
		{
			return nullptr;
		}
		//Create a fbx asset import data and fill the options
		UInterchangeAssetImportData* DestinationData = NewObject<UInterchangeAssetImportData>(Obj);
		//Transfer the Source file information
		TransferSourceFileInformation(FbxAssetImportData, DestinationData);

		//Create a container
		UInterchangeBaseNodeContainer* DestinationContainer = NewObject<UInterchangeBaseNodeContainer>(DestinationData);
		DestinationData->SetNodeContainer(DestinationContainer);
		const FString BasePathToRemove = FPaths::GetBaseFilename(FbxAssetImportData->GetFirstFilename()) + TEXT("_");
		FString NodeDisplayLabel = Obj->GetName();
		if (NodeDisplayLabel.StartsWith(BasePathToRemove))
		{
			NodeDisplayLabel.RightInline(NodeDisplayLabel.Len() - BasePathToRemove.Len());
		}
		const FString NodeUniqueId = FGuid::NewGuid().ToString(EGuidFormats::Base36Encoded) + TEXT("_") + NodeDisplayLabel;

		TArray<UObject*> Pipelines;
		UInterchangeGenericAssetsPipeline* GenericAssetPipeline = GetDefaultGenericAssetPipelineForConvertion(DestinationData);
		Pipelines.Add(GenericAssetPipeline);
		DestinationData->SetPipelines(Pipelines);

		GenericAssetPipeline->ReimportStrategy = EReimportStrategyFlags::ApplyNoProperties;
		GenericAssetPipeline->ImportOffsetRotation = FbxAssetImportData->ImportRotation;
		GenericAssetPipeline->ImportOffsetTranslation = FbxAssetImportData->ImportTranslation;
		GenericAssetPipeline->ImportOffsetUniformScale = FbxAssetImportData->ImportUniformScale;

		UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings = NewObject<UInterchangeFbxTranslatorSettings>(DestinationData);
		InterchangeFbxTranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		InterchangeFbxTranslatorSettings->bConvertScene = FbxAssetImportData->bConvertScene;
		InterchangeFbxTranslatorSettings->bForceFrontXAxis = FbxAssetImportData->bForceFrontXAxis;
		InterchangeFbxTranslatorSettings->bConvertSceneUnit = FbxAssetImportData->bConvertSceneUnit;
		InterchangeFbxTranslatorSettings->bKeepFbxNamespace = GetDefault<UEditorPerProjectUserSettings>()->bKeepFbxNamespace;
		DestinationData->SetTranslatorSettings(InterchangeFbxTranslatorSettings);
		bool bConvertToNewType = false;
		if (const UFbxStaticMeshImportData* LegacyStaticMeshImportData = Cast<UFbxStaticMeshImportData>(FbxAssetImportData))
		{
			UInterchangeStaticMeshFactoryNode* MeshNode = NewObject<UInterchangeStaticMeshFactoryNode>(DestinationContainer);
			MeshNode->InitializeStaticMeshNode(NodeUniqueId, NodeDisplayLabel, UStaticMesh::StaticClass()->GetName(), DestinationContainer);

			GenericAssetPipeline->MeshPipeline->CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_StaticMesh;
			if (Obj->IsA<UStaticMesh>())
			{
				FillInterchangeGenericAssetsPipelineFromFbxStaticMesh(GenericAssetPipeline, Cast<UStaticMesh>(Obj));
			}
			else
			{
				bConvertToNewType = true;
			}
			FillInterchangeGenericAssetsPipelineFromFbxStaticMeshImportData(GenericAssetPipeline
				, LegacyStaticMeshImportData);
		}
		else if (const UFbxSkeletalMeshImportData* LegacySkeletalMeshImportData = Cast<UFbxSkeletalMeshImportData>(FbxAssetImportData))
		{
			UInterchangeSkeletalMeshFactoryNode* MeshNode = NewObject<UInterchangeSkeletalMeshFactoryNode>(DestinationContainer);
			MeshNode->InitializeSkeletalMeshNode(NodeUniqueId, NodeDisplayLabel, USkeletalMesh::StaticClass()->GetName(), DestinationContainer);

			GenericAssetPipeline->MeshPipeline->CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_SkeletalMesh;
			FillInterchangeGenericAssetsPipelineFromFbxSkeletalMeshImportData(GenericAssetPipeline
				, LegacySkeletalMeshImportData);
			
			bConvertToNewType = (!Obj->IsA<USkeletalMesh>());
		}
		else if (const UFbxAnimSequenceImportData* LegacyAnimSequenceImportData = Cast<UFbxAnimSequenceImportData>(FbxAssetImportData))
		{
			UInterchangeAnimSequenceFactoryNode* AnimationNode = NewObject<UInterchangeAnimSequenceFactoryNode>(DestinationContainer);
			AnimationNode->InitializeAnimSequenceNode(NodeUniqueId, NodeDisplayLabel, DestinationContainer);

			FillInterchangeGenericAssetsPipelineFromFbxAnimSequenceImportData(GenericAssetPipeline
				, LegacyAnimSequenceImportData);

			bConvertToNewType = (!Obj->IsA<UAnimSequence>());
		}

		if (UInterchangeFactoryBaseNode* DestinationFactoryNode = DestinationContainer->GetFactoryNode(NodeUniqueId))
		{
			DestinationFactoryNode->SetReimportStrategyFlags(EReimportStrategyFlags::ApplyNoProperties);
			DestinationFactoryNode->SetCustomReferenceObject(Obj);
			DestinationData->SetNodeContainer(DestinationContainer);
			DestinationData->NodeUniqueID = NodeUniqueId;
		}
#if WITH_EDITOR
		//If the type of asset has change we must convert the options
		if (bConvertToNewType)
		{
			DestinationData->ConvertAssetImportDataToNewOwner(Obj);
		}
#endif
		return DestinationData;
	}

	UAssetImportData* ConvertToInterchange(UObject* Owner, const UFbxImportUI* FbxImportUI)
	{
		if (!FbxImportUI || !Owner)
		{
			return nullptr;
		}
		//Create a fbx asset import data and fill the options
		UInterchangeAssetImportData* DestinationData = NewObject<UInterchangeAssetImportData>(Owner);

		//Create a node container
		UInterchangeBaseNodeContainer* DestinationContainer = NewObject<UInterchangeBaseNodeContainer>(DestinationData);
		DestinationData->SetNodeContainer(DestinationContainer);

		TArray<UObject*> Pipelines;
		UInterchangeGenericAssetsPipeline* GenericAssetPipeline = GetDefaultGenericAssetPipelineForConvertion(DestinationData);
		Pipelines.Add(GenericAssetPipeline);
		DestinationData->SetPipelines(Pipelines);

		auto SetTranslatorSettings = [&DestinationData](const UFbxAssetImportData* FbxAssetImportData)
			{
				UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings = NewObject<UInterchangeFbxTranslatorSettings>(DestinationData);
				InterchangeFbxTranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
				InterchangeFbxTranslatorSettings->bConvertScene = FbxAssetImportData->bConvertScene;
				InterchangeFbxTranslatorSettings->bForceFrontXAxis = FbxAssetImportData->bForceFrontXAxis;
				InterchangeFbxTranslatorSettings->bConvertSceneUnit = FbxAssetImportData->bConvertSceneUnit;
				InterchangeFbxTranslatorSettings->bKeepFbxNamespace = GetDefault<UEditorPerProjectUserSettings>()->bKeepFbxNamespace;
				DestinationData->SetTranslatorSettings(InterchangeFbxTranslatorSettings);
			};

		//General Options
		GenericAssetPipeline->bUseSourceNameForAsset = FbxImportUI->bOverrideFullName;
		GenericAssetPipeline->ReimportStrategy = EReimportStrategyFlags::ApplyNoProperties;

		//Material Options
		GenericAssetPipeline->MaterialPipeline->bImportMaterials = FbxImportUI->bImportMaterials;
		GenericAssetPipeline->MaterialPipeline->bReuseExistingMaterials = true;
		switch (FbxImportUI->TextureImportData->MaterialSearchLocation)
		{
			case EMaterialSearchLocation::Local:
				GenericAssetPipeline->MaterialPipeline->SearchLocation = EInterchangeMaterialSearchLocation::Local;
				break;
			case EMaterialSearchLocation::UnderParent:
				GenericAssetPipeline->MaterialPipeline->SearchLocation = EInterchangeMaterialSearchLocation::UnderParent;
				break;
			case EMaterialSearchLocation::UnderRoot:
				GenericAssetPipeline->MaterialPipeline->SearchLocation = EInterchangeMaterialSearchLocation::UnderRoot;
				break;
			case EMaterialSearchLocation::AllAssets:
				GenericAssetPipeline->MaterialPipeline->SearchLocation = EInterchangeMaterialSearchLocation::AllAssets;
				break;
			case EMaterialSearchLocation::DoNotSearch:
				{
					GenericAssetPipeline->MaterialPipeline->bReuseExistingMaterials = false;
				}
				break;
			default:
				ensure(false);
				break;
		}
		if (FbxImportUI->TextureImportData->BaseMaterialName.IsAsset())
		{
			GenericAssetPipeline->MaterialPipeline->MaterialImport = EInterchangeMaterialImportOption::ImportAsMaterialInstances;
			GenericAssetPipeline->MaterialPipeline->ParentMaterial = FbxImportUI->TextureImportData->BaseMaterialName;
		}
		else
		{
			GenericAssetPipeline->MaterialPipeline->MaterialImport = EInterchangeMaterialImportOption::ImportAsMaterials;
			GenericAssetPipeline->MaterialPipeline->ParentMaterial.Reset();
		}

		//Texture Options
		GenericAssetPipeline->MaterialPipeline->TexturePipeline->bImportTextures = FbxImportUI->bImportTextures;
		GenericAssetPipeline->MaterialPipeline->TexturePipeline->bFlipNormalMapGreenChannel = FbxImportUI->TextureImportData->bInvertNormalMaps;

		//Default the force animation to false
		GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations = false;

		//Discover if we must import something in particular
		if (FbxImportUI->MeshTypeToImport == EFBXImportType::FBXIT_SkeletalMesh
			|| (FbxImportUI->bImportAsSkeletal && FbxImportUI->bImportMesh))
		{
			GenericAssetPipeline->MeshPipeline->bImportSkeletalMeshes = true;
			GenericAssetPipeline->MeshPipeline->CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_SkeletalMesh;

			if (FbxImportUI->Skeleton)
			{
				GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->Skeleton = FbxImportUI->Skeleton;
			}

			GenericAssetPipeline->AnimationPipeline->bImportAnimations = FbxImportUI->bImportAnimations;

			GenericAssetPipeline->ImportOffsetRotation = FbxImportUI->SkeletalMeshImportData->ImportRotation;
			GenericAssetPipeline->ImportOffsetTranslation = FbxImportUI->SkeletalMeshImportData->ImportTranslation;
			GenericAssetPipeline->ImportOffsetUniformScale = FbxImportUI->SkeletalMeshImportData->ImportUniformScale;

			SetTranslatorSettings(FbxImportUI->SkeletalMeshImportData);

			FillInterchangeGenericAssetsPipelineFromFbxMeshImportData(GenericAssetPipeline, Cast<UFbxSkeletalMeshImportData>(FbxImportUI->SkeletalMeshImportData));
		}
		else if (FbxImportUI->MeshTypeToImport == EFBXImportType::FBXIT_StaticMesh)
		{
			GenericAssetPipeline->MeshPipeline->bImportStaticMeshes = true;
			GenericAssetPipeline->MeshPipeline->CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_StaticMesh;

			GenericAssetPipeline->AnimationPipeline->bImportAnimations = false;
			GenericAssetPipeline->MeshPipeline->bImportSkeletalMeshes = false;

			GenericAssetPipeline->ImportOffsetRotation = FbxImportUI->StaticMeshImportData->ImportRotation;
			GenericAssetPipeline->ImportOffsetTranslation = FbxImportUI->StaticMeshImportData->ImportTranslation;
			GenericAssetPipeline->ImportOffsetUniformScale = FbxImportUI->StaticMeshImportData->ImportUniformScale;

			SetTranslatorSettings(FbxImportUI->StaticMeshImportData);

			FillInterchangeGenericAssetsPipelineFromFbxMeshImportData(GenericAssetPipeline, Cast<UFbxStaticMeshImportData>(FbxImportUI->StaticMeshImportData));
		}
		else if (FbxImportUI->MeshTypeToImport == EFBXImportType::FBXIT_Animation || (FbxImportUI->bImportAsSkeletal && !FbxImportUI->bImportMesh && FbxImportUI->bImportAnimations))
		{
			GenericAssetPipeline->AnimationPipeline->bImportAnimations = true;
			if (FbxImportUI->Skeleton)
			{
				GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations = true;
				GenericAssetPipeline->CommonSkeletalMeshesAndAnimationsProperties->Skeleton = FbxImportUI->Skeleton;

				GenericAssetPipeline->MeshPipeline->bImportStaticMeshes = false;
				GenericAssetPipeline->MeshPipeline->bImportSkeletalMeshes = false;
			}
			else
			{
				GenericAssetPipeline->MeshPipeline->bImportStaticMeshes = true;
				GenericAssetPipeline->MeshPipeline->bImportSkeletalMeshes = true;
			}

			GenericAssetPipeline->ImportOffsetRotation = FbxImportUI->AnimSequenceImportData->ImportRotation;
			GenericAssetPipeline->ImportOffsetTranslation = FbxImportUI->AnimSequenceImportData->ImportTranslation;
			GenericAssetPipeline->ImportOffsetUniformScale = FbxImportUI->AnimSequenceImportData->ImportUniformScale;

			SetTranslatorSettings(FbxImportUI->AnimSequenceImportData);

			FillInterchangeGenericAssetsPipelineFromFbxAnimSequenceImportData(GenericAssetPipeline, Cast<UFbxAnimSequenceImportData>(FbxImportUI->AnimSequenceImportData));
		}
		else
		{
			//Allow importing all type
			GenericAssetPipeline->MeshPipeline->CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_None;
			GenericAssetPipeline->MeshPipeline->bImportStaticMeshes = true;
			GenericAssetPipeline->MeshPipeline->bImportSkeletalMeshes = true;
			GenericAssetPipeline->AnimationPipeline->bImportAnimations = true;

			SetTranslatorSettings(FbxImportUI->StaticMeshImportData);

			//Use the static mesh data
			FillInterchangeGenericAssetsPipelineFromFbxMeshImportData(GenericAssetPipeline, Cast<UFbxStaticMeshImportData>(FbxImportUI->StaticMeshImportData));
		}

		if (const UFbxStaticMeshImportData* LegacyStaticMeshImportData = Cast<UFbxStaticMeshImportData>(FbxImportUI->StaticMeshImportData))
		{
			FillInterchangeGenericAssetsPipelineFromFbxStaticMeshImportData(GenericAssetPipeline
				, LegacyStaticMeshImportData
				, false);
		}
		if (const UFbxSkeletalMeshImportData* LegacySkeletalMeshImportData = Cast<UFbxSkeletalMeshImportData>(FbxImportUI->SkeletalMeshImportData))
		{
			FillInterchangeGenericAssetsPipelineFromFbxSkeletalMeshImportData(GenericAssetPipeline
				, LegacySkeletalMeshImportData
				, false);
		}
		if (const UFbxAnimSequenceImportData* LegacyAnimSequenceImportData = Cast<UFbxAnimSequenceImportData>(FbxImportUI->AnimSequenceImportData))
		{
			FillInterchangeGenericAssetsPipelineFromFbxAnimSequenceImportData(GenericAssetPipeline
				, LegacyAnimSequenceImportData);
		}
		return DestinationData;
	}

	UAssetImportData* ConvertData(UObject* Obj, UAssetImportData* SourceData, const bool bInterchangeSupportTargetExtension)
	{
		if (const UInterchangeAssetImportData* InterchangeSourceData = Cast<UInterchangeAssetImportData>(SourceData))
		{
			if (bInterchangeSupportTargetExtension)
			{
				//This converter do not convert Interchange to Interchange
				return nullptr;
			}

			//Do not convert scene data
			if (InterchangeSourceData->SceneImportAsset.IsValid())
			{
				return nullptr;
			}

			//Convert Interchange import data to Legacy Fbx Import data
			if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Obj))
			{
				return ConvertToLegacyFbx(StaticMesh, InterchangeSourceData);
			}
			else if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Obj))
			{
				return ConvertToLegacyFbx(SkeletalMesh, InterchangeSourceData);
			}
			else if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(Obj))
			{
				return ConvertToLegacyFbx(AnimSequence, InterchangeSourceData);
			}
		}
		else if (const UFbxAssetImportData* LegacyFbxSourceData = Cast<UFbxAssetImportData>(SourceData))
		{
			if (!bInterchangeSupportTargetExtension)
			{
				//This converter do not convert Legacy Fbx to other format then Interchange.
				//This is probably a conversion from Legacy Fbx to Legacy Fbx which we do not need to do
				return nullptr;
			}

			//Do not convert scene data
			if (LegacyFbxSourceData->bImportAsScene)
			{
				return nullptr;
			}

			//Convert Legacy Fbx import data to Interchange Import data
			return ConvertToInterchange(Obj, LegacyFbxSourceData);
		}
		return nullptr;
	}
} //ns: UE::Interchange::Private

bool UInterchangeFbxAssetImportDataConverter::ConvertImportData(UObject* Asset, const FString& TargetExtension) const
{
	bool bResult = false;
	const FString TargetExtensionLower = TargetExtension.ToLower();
	bool bUseInterchangeFramework = UInterchangeManager::IsInterchangeImportEnabled();;
	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	
	UAssetImportData* OldAssetData = nullptr;
	TArray<FString> InterchangeSupportedExtensions;
	if (Asset->IsA(UStaticMesh::StaticClass()) || Asset->IsA(USkeletalMesh::StaticClass()))
	{
		InterchangeSupportedExtensions = InterchangeManager.GetSupportedAssetTypeFormats(EInterchangeTranslatorAssetType::Meshes);
	}
	else if (Asset->IsA(UAnimSequence::StaticClass()))
	{
		InterchangeSupportedExtensions = InterchangeManager.GetSupportedAssetTypeFormats(EInterchangeTranslatorAssetType::Animations);
	}
	//Remove the detail of the extensions
	for (FString& Extension : InterchangeSupportedExtensions)
	{
		int32 FindIndex = INDEX_NONE;
		Extension.FindChar(';', FindIndex);
		if (FindIndex != INDEX_NONE && FindIndex < Extension.Len() && FindIndex > 0)
		{
			Extension.LeftInline(FindIndex);
		}
	}
	const bool bInterchangeSupportTargetExtension = bUseInterchangeFramework && InterchangeSupportedExtensions.Contains(TargetExtensionLower);
	
	if (TargetExtensionLower.Equals(TEXT("fbx")) || bInterchangeSupportTargetExtension)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
		{
			if (UAssetImportData* ConvertedAssetData = UE::Interchange::Private::ConvertData(StaticMesh, StaticMesh->GetAssetImportData(), bInterchangeSupportTargetExtension))
			{
				OldAssetData = StaticMesh->GetAssetImportData();
				StaticMesh->SetAssetImportData(ConvertedAssetData);
				bResult = true;
			}
		}
		else if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))
		{
			if (UAssetImportData* ConvertedAssetData = UE::Interchange::Private::ConvertData(SkeletalMesh, SkeletalMesh->GetAssetImportData(), bInterchangeSupportTargetExtension))
			{
				OldAssetData = SkeletalMesh->GetAssetImportData();
				SkeletalMesh->SetAssetImportData(ConvertedAssetData);
				bResult = true;
			}
		}
		else if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset))
		{
			if (UAssetImportData* ConvertedAssetData = UE::Interchange::Private::ConvertData(AnimSequence, AnimSequence->AssetImportData, bInterchangeSupportTargetExtension))
			{
				OldAssetData = AnimSequence->AssetImportData;
				AnimSequence->AssetImportData = ConvertedAssetData;
				bResult = true;
			}
		}
	}
	
	//Make sure old import asset data will be deleted by the next garbage collect
	if (bResult && OldAssetData)
	{
		OldAssetData->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		OldAssetData->ClearFlags(RF_Public | RF_Standalone);
	}

	return bResult;
}

bool UInterchangeFbxAssetImportDataConverter::ConvertImportData(const UObject* SourceImportData, const UClass* DestinationClass, UObject** DestinationImportData) const
{
	bool bResult = false;
	if (!SourceImportData || !DestinationImportData)
	{
		return bResult;
	}

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	if (SourceImportData->IsA<UInterchangeAssetImportData>())
	{
		// UE-359435: Make sure UFbxImportUI is created using Transient package to prevent Editor-only dependencies.
		if (UFbxImportUI* FbxImportUI = UE::Interchange::Private::ConvertToLegacyFbx(GetTransientPackage(), Cast<UInterchangeAssetImportData>(SourceImportData)))
		{
			UObject* DestinationPtr = nullptr;

			if (DestinationClass->IsChildOf<UFbxImportUI>())
			{
				DestinationPtr = FbxImportUI;
			}
			else
			{
				if (DestinationClass->IsChildOf<UFbxStaticMeshImportData>())
				{
					DestinationPtr = FbxImportUI->StaticMeshImportData;
				}
				else if (DestinationClass->IsChildOf<UFbxSkeletalMeshImportData>())
				{
					DestinationPtr = FbxImportUI->SkeletalMeshImportData;
				}
				else if (DestinationClass->IsChildOf<UFbxAnimSequenceImportData>())
				{
					DestinationPtr = FbxImportUI->AnimSequenceImportData;
				}

				if (DestinationPtr)
				{
					//Don't forget to set proper outer for supported classes.
					DestinationPtr->Rename(nullptr, SourceImportData->GetOuter(), REN_DontCreateRedirectors | REN_NonTransactional);
				}
			}

			if (DestinationPtr)
			{
				(*DestinationImportData) = DestinationPtr;
				bResult = true;
			}
			else
			{
				bResult = false;
			}
		}
	}
	else
	{
		const UFbxImportUI* FbxImportUI = nullptr;
		if (SourceImportData->IsA<UFbxImportUI>())
		{
			FbxImportUI = const_cast<UFbxImportUI*>(Cast<UFbxImportUI>(SourceImportData));
			
		}
		else if (SourceImportData->IsA<UFbxAssetImportData>())
		{
			//We convert the UFbxAssetImportData into a UFbxImportUI
			auto FillFbxAssetImportData = [](const UFbxAssetImportData* SourceAssetImportData, UFbxAssetImportData* DestinationAssetImportData)
				{
					DestinationAssetImportData->bConvertScene = SourceAssetImportData->bConvertScene;
					DestinationAssetImportData->bConvertSceneUnit = SourceAssetImportData->bConvertSceneUnit;
					DestinationAssetImportData->bForceFrontXAxis = SourceAssetImportData->bForceFrontXAxis;
					DestinationAssetImportData->bImportAsScene = SourceAssetImportData->bImportAsScene;
					DestinationAssetImportData->ImportRotation = SourceAssetImportData->ImportRotation;
					DestinationAssetImportData->ImportTranslation = SourceAssetImportData->ImportTranslation;
					DestinationAssetImportData->ImportUniformScale = SourceAssetImportData->ImportUniformScale;
				};

			UFbxImportUI* TempFbxImportUI = NewObject<UFbxImportUI>(GetTransientPackage());
			TempFbxImportUI->bImportMaterials = false;
			TempFbxImportUI->bImportAsSkeletal = false;
			TempFbxImportUI->bImportMesh = false;
			TempFbxImportUI->bImportAnimations = false;
			TempFbxImportUI->bImportRigidMesh = false;
			TempFbxImportUI->bImportTextures = false;
			TempFbxImportUI->bIsObjImport = false;
			TempFbxImportUI->bIsReimport = false;
			TempFbxImportUI->bCreatePhysicsAsset = false;
			TempFbxImportUI->PhysicsAsset = nullptr;
			TempFbxImportUI->Skeleton = nullptr;
			if (SourceImportData->IsA<UFbxSkeletalMeshImportData>())
			{
				TempFbxImportUI->SkeletalMeshImportData = const_cast<UFbxSkeletalMeshImportData*>(Cast<UFbxSkeletalMeshImportData>(SourceImportData));
				TempFbxImportUI->MeshTypeToImport = EFBXImportType::FBXIT_SkeletalMesh;
				TempFbxImportUI->bImportAsSkeletal = true;
				TempFbxImportUI->bImportMesh = true;

				FillFbxAssetImportData(TempFbxImportUI->SkeletalMeshImportData, TempFbxImportUI->StaticMeshImportData);
				FillFbxAssetImportData(TempFbxImportUI->SkeletalMeshImportData, TempFbxImportUI->AnimSequenceImportData);
				FillFbxAssetImportData(TempFbxImportUI->SkeletalMeshImportData, TempFbxImportUI->TextureImportData);
			}
			else if (SourceImportData->IsA<UFbxStaticMeshImportData>())
			{
				TempFbxImportUI->MeshTypeToImport = EFBXImportType::FBXIT_StaticMesh;
				TempFbxImportUI->bImportMesh = true;

				TempFbxImportUI->StaticMeshImportData = const_cast<UFbxStaticMeshImportData*>(Cast<UFbxStaticMeshImportData>(SourceImportData));
				FillFbxAssetImportData(TempFbxImportUI->StaticMeshImportData, TempFbxImportUI->SkeletalMeshImportData);
				FillFbxAssetImportData(TempFbxImportUI->StaticMeshImportData, TempFbxImportUI->AnimSequenceImportData);
				FillFbxAssetImportData(TempFbxImportUI->StaticMeshImportData, TempFbxImportUI->TextureImportData);
			}
			else if (SourceImportData->IsA<UFbxAnimSequenceImportData>())
			{
				TempFbxImportUI->MeshTypeToImport = EFBXImportType::FBXIT_Animation;
				TempFbxImportUI->bImportAsSkeletal = true;
				TempFbxImportUI->bImportMesh = false;
				TempFbxImportUI->AnimSequenceImportData = const_cast<UFbxAnimSequenceImportData*>(Cast<UFbxAnimSequenceImportData>(SourceImportData));
				FillFbxAssetImportData(TempFbxImportUI->AnimSequenceImportData, TempFbxImportUI->SkeletalMeshImportData);
				FillFbxAssetImportData(TempFbxImportUI->AnimSequenceImportData, TempFbxImportUI->StaticMeshImportData);
				FillFbxAssetImportData(TempFbxImportUI->AnimSequenceImportData, TempFbxImportUI->TextureImportData);
			}
			else
			{
				ensureMsgf(false, TEXT("Fbx interchange converter: miss match between CanConvertClass and the convertion capacity"));
				TempFbxImportUI = nullptr;
			}
			//Assign to the const pointer we use to convert the data
			FbxImportUI = TempFbxImportUI;
		}

		if (FbxImportUI)
		{
			//Convert Legacy Fbx to Interchange
			*DestinationImportData = UE::Interchange::Private::ConvertToInterchange(SourceImportData->GetOuter(), FbxImportUI);
			bResult = true;
		}
	}
	return bResult;
}

bool UInterchangeFbxAssetImportDataConverter::CanConvertClass(const UClass* SourceClass, const UClass* DestinationClass) const
{
	if (SourceClass->IsChildOf(UFbxImportUI::StaticClass()))
	{
		return DestinationClass->IsChildOf(UInterchangeAssetImportData::StaticClass());
	}

	if (SourceClass->IsChildOf(UFbxAssetImportData::StaticClass()))
	{
		if (SourceClass->IsChildOf(UFbxSkeletalMeshImportData::StaticClass())
			|| SourceClass->IsChildOf(UFbxStaticMeshImportData::StaticClass())
			|| SourceClass->IsChildOf(UFbxAnimSequenceImportData::StaticClass()))
		{
			return DestinationClass->IsChildOf(UInterchangeAssetImportData::StaticClass());
		}
	}

	if (SourceClass->IsChildOf(UInterchangeAssetImportData::StaticClass()))
	{
		if (DestinationClass->IsChildOf(UFbxImportUI::StaticClass())
			|| DestinationClass->IsChildOf(UFbxSkeletalMeshImportData::StaticClass())
			|| DestinationClass->IsChildOf(UFbxStaticMeshImportData::StaticClass())
			|| DestinationClass->IsChildOf(UFbxAnimSequenceImportData::StaticClass()))
		{
			return true;
		}
	}
	return false;
}
