// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangePipelineSettingsCacheHandler.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CoreGlobals.h"
#include "Containers/Array.h"
#include "InterchangeEditorModule.h"
#include "InterchangePipelineBase.h"
#include "InterchangeEditorLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"

TSet<uint32> FInterchangePipelineSettingsCacheHandler::CachedPipelineHashes;
FDelegateHandle FInterchangePipelineSettingsCacheHandler::AssetRemovedHandle;

void FInterchangePipelineSettingsCacheHandler::InitializeCacheHandler()
{
	IAssetRegistry* AssetRegistry = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	AssetRemovedHandle = AssetRegistry->OnAssetRemoved().AddStatic(&FInterchangePipelineSettingsCacheHandler::OnAssetRemoved);
}

void FInterchangePipelineSettingsCacheHandler::OnAssetRemoved(const FAssetData& RemovedAsset)
{
	if (UClass* AssetClass = RemovedAsset.GetClass(); !AssetClass || !AssetClass->IsChildOf<UInterchangePipelineBase>())
	{
		return;
	}

	const FString PipelinePathString = RemovedAsset.GetObjectPathString();
	const uint32 PipelinePathHash = GetTypeHash(PipelinePathString);
	const FString PathHashString = FString::Printf(TEXT("%u"), PipelinePathHash);
	if (GConfig->EmptySectionsMatchingString(*PathHashString, UInterchangePipelineBase::GetDefaultConfigFileName()))
	{
		UE_LOGF(LogInterchangeEditor, Log, "Cached pipeline settings are removed for %ls", *RemovedAsset.AssetName.ToString());
		constexpr bool bRemoveFromCache = false;
		GConfig->Flush(bRemoveFromCache, UInterchangePipelineBase::GetDefaultConfigFileName());
	}
}

void FInterchangePipelineSettingsCacheHandler::ShutdownCacheHandler()
{
	if (AssetRemovedHandle.IsValid() && FModuleManager::Get().IsModuleLoaded(AssetRegistryConstants::ModuleName))
	{
		IAssetRegistry* AssetRegistry = &FModuleManager::GetModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
		AssetRegistry->OnAssetRemoved().Remove(AssetRemovedHandle);
	}

	CachedPipelineHashes.Empty();
}
