// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Set.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/SoftObjectPath.h"

class FInterchangePipelineSettingsCacheHandler
{
public:
	static void InitializeCacheHandler();

	static void OnAssetRemoved(const FAssetData& RemovedAsset);

	static void ShutdownCacheHandler();

private:
	static TSet<uint32> CachedPipelineHashes;

	static FDelegateHandle AssetRemovedHandle;
};
