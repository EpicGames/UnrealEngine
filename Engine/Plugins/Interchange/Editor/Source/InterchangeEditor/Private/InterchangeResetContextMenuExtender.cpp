// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeResetContextMenuExtender.h"

#include "Algo/AnyOf.h"
#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/IConsoleManager.h"
#include "InterchangeAssetUserData.h"
#include "InterchangeEditorScriptLibrary.h"
#include "InterchangeImportReset.h"
#include "InterchangeLevelFactoryNode.h"
#include "InterchangeManager.h"
#include "InterchangeSceneImportAsset.h"
#include "LevelEditor.h"
#include "LevelEditorMenuContext.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceEditorPivotActor.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "InterchangeResetContextMenuHandler"

namespace UE::Interchange::InterchangeReset
{
	TSharedRef<FExtender> OnExtendLevelEditorActorSelectionMenu(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors)
	{
		TSharedRef<FExtender> Extender = MakeShared<FExtender>();

		bool bShouldExtendActorActions = false;

		for (const AActor* Actor : SelectedActors)
		{
			if (UInterchangeEditorScriptLibrary::CanResetActor(Actor))
			{
				bShouldExtendActorActions = true;
				break;
			}
		}

		if (bShouldExtendActorActions)
		{
			// Add the Interchange actions sub-menu extender
			Extender->AddMenuExtension("ActorTypeTools", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
				[SelectedActors](FMenuBuilder& MenuBuilder)
				{
					MenuBuilder.BeginSection("Interchange", LOCTEXT("InterchangeMenuSection", "Interchange"));
					MenuBuilder.AddMenuEntry(
						NSLOCTEXT("InterchangeActions", "ObjectContext_ResetInterchange", "Reset Properties"),
						NSLOCTEXT("InterchangeActions", "ObjectContext_ResetInterchangeTooltip", "Resets overridden values with the values from Interchange Import"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh"),
						FUIAction(FExecuteAction::CreateStatic(&UInterchangeEditorScriptLibrary::ResetActors, SelectedActors), FCanExecuteAction()));
					MenuBuilder.EndSection();
				}));
		}

		return Extender;
	}

	namespace LevelContextMenuHelpers
	{
		bool CanExecuteResetLevel(const FToolMenuContext& InContext)
		{
			const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
			TArray<UWorld*> Worlds = Context->LoadSelectedObjects<UWorld>();
			return Algo::AnyOf(Worlds, [](const UWorld* World)
				{
					return UInterchangeEditorScriptLibrary::CanResetWorld(World);
				});
		}

		void ExecuteResetLevel(const FToolMenuContext& InContext)
		{
			const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
			TArray<UWorld*> Worlds = Context->LoadSelectedObjects<UWorld>();

			for (UWorld* World : Worlds)
			{
				UInterchangeEditorScriptLibrary::ResetLevelAsset(World);	
			}
		}
	}
} //ns UE::Interchange::InterchangeReset

static FDelayedAutoRegisterHelper DelayedAutoRegisterLevelContextMenu(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.InterchangeReset"), false);
			const bool bInterchangeResetEnabled = CVar ? CVar->GetBool() : false;
			if (!bInterchangeResetEnabled)
			{
				return;
			}

			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UWorld::StaticClass());

			if (!Menu->FindSection("Interchange"))
			{
				const FToolMenuInsert MenuInsert(NAME_None, EToolMenuInsertType::First);
				Menu->AddSection("Interchange", LOCTEXT("Level_ResetSection", "Interchange"), MenuInsert);
			}
			FToolMenuSection& Section = Menu->FindOrAddSection("Interchange");

			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection.Context);

					if (Context->SelectedAssets.Num() != 1)
					{
						return;
					}

					bool bAddResetSection = false;

					const FAssetData& WorldAsset = Context->SelectedAssets[0];
					if (Context->SelectedAssets[0].IsInstanceOf<UWorld>() && Context->SelectedAssets[0].IsAssetLoaded())
					{
						if (UWorld* World = Cast<UWorld>(Context->SelectedAssets[0].GetAsset()))
						{
							if (UInterchangeEditorScriptLibrary::CanResetWorld(World))
							{
								bAddResetSection = true;
							}
						}
					}

					if (bAddResetSection)
					{
						using namespace UE::Interchange::InterchangeReset;

						const TAttribute<FText> Label = LOCTEXT("Level_ResetScene", "Reset Properties");
						const TAttribute<FText> ToolTip = LOCTEXT("Level_ResetSceneTooltip", "Reset the level to original pipeline properties.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh");

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&LevelContextMenuHelpers::ExecuteResetLevel);
						UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&LevelContextMenuHelpers::CanExecuteResetLevel);

						InSection.AddMenuEntry("Level_ResetScene", Label, ToolTip, Icon, UIAction);
					}
				}));
		}));
	});

FDelegateHandle FInterchangeResetContextMenuExtender::LevelEditorExtenderDelegateHandle;

void FInterchangeResetContextMenuExtender::SetupLevelEditorContextMenuExtender()
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.InterchangeReset"), false);
	const bool bInterchangeResetEnabled = CVar ? CVar->GetBool() : false;
	if (!bInterchangeResetEnabled)
	{
		return;
	}

	if (!LevelEditorExtenderDelegateHandle.IsValid())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked< FLevelEditorModule >("LevelEditor");
		TArray< FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors >& CBMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

		CBMenuExtenderDelegates.Add(FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateStatic(&UE::Interchange::InterchangeReset::OnExtendLevelEditorActorSelectionMenu));
		LevelEditorExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
	}
}

void FInterchangeResetContextMenuExtender::RemoveLevelEditorContextMenuExtender()
{
	if (LevelEditorExtenderDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked< FLevelEditorModule >("LevelEditor");
		TArray< FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors >& CBMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
		CBMenuExtenderDelegates.RemoveAll(
			[&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate)
			{
				return Delegate.GetHandle() == LevelEditorExtenderDelegateHandle;
			}
		);
		LevelEditorExtenderDelegateHandle.Reset();
	}
}

#undef LOCTEXT_NAMESPACE // "InterchangeResetContextMenuHandler"