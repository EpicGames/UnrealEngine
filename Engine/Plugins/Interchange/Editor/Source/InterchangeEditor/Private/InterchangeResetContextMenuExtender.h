// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/CoreDelegates.h"

class FInterchangeResetContextMenuExtender
{
public:
	static void SetupLevelEditorContextMenuExtender();
	static void RemoveLevelEditorContextMenuExtender();

private:
	static FDelegateHandle LevelEditorExtenderDelegateHandle;
};
