// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define INTERCHANGEEDITOR_MODULE_NAME TEXT("InterchangeEditor")

/**
 * This module allows out-of-process Interchange translators in case a third-party SDK is not thread-safe.
 */
class FInterchangeEditorModule : public IModuleInterface
{
public:
	static FInterchangeEditorModule& Get();
	static bool IsAvailable();

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
