// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Containers/Array.h"

#include "InterchangeEditorScriptLibrary.generated.h"

#define UE_API INTERCHANGEEDITOR_API

class AActor;
class ALevelInstance;

class UWorld;
class UInterchangeSceneImportAsset;

UCLASS(MinimalAPI)
class UInterchangeEditorScriptLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	 * Performs Interchange Reset on a Level Asset.
	 * @param World is the level asset to reset.
	 */
	UFUNCTION(BlueprintCallable, Category="Interchange Utilities | Reset")
	static UE_API void ResetLevelAsset(UWorld* World);

	/**
	 * Performs Interchange Reset on an Interchange Scene Import Asset.
	 * Resets all the actors added to the level and assets imported.
	 * @param SceneImportAsset to reset
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Reset")
	static UE_API void ResetSceneImportAsset(UInterchangeSceneImportAsset* SceneImportAsset);

	/**
	 * Performs Interchange Reset on Actors.
	 * Resets all qualifying actors. Does nothing to actors that cannot be reset.
	 * @param Actors array containing actors to reset
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Reset")
	static UE_API void ResetActors(TArray<AActor*> Actors);

	/**
	 * Checks if an actor can be reset.
	 * @param Actor to check
	 * @return can actor be reset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Reset")
	static UE_API bool CanResetActor(const AActor* Actor);

	/**
	 * Checks if an world can be reset.
	 * @param World to check
	 * @return can world be reset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Reset")
	static UE_API bool CanResetWorld(const UWorld* World);

	/**
	 * Make Level Instance Actor editable.
	 * @param LevelInstance actor to edit
	 * @return whether putting level instance in edit mode was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Reset")
	static UE_API bool LevelInstanceEnterEditMode(ALevelInstance* LevelInstance);

	/**
	 * Apply/Discard the changes to Level Instance Actor.
	 * @param LevelInstance actor to commit
	 * @param bDiscardChanges should changes be applied or discarded.
	 * @return whether committing level instance was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Reset")
	static UE_API bool LevelInstanceCommit(ALevelInstance* LevelInstance, bool bDiscardChanges);

	/**
	 * Returns array of actors that are editable in the editor when the level instance is put in edit mode.
	 * NOTE: This will return a non-empty array if the LevelInstance is put in the edit mode.
	 *
	 * @param LevelInstance to get editable actors from
	 * @return Array of editable actors.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Reset")
	static UE_API const TArray<AActor*> LevelInstanceGetEditableActors(ALevelInstance* LevelInstance);

	/**
	 * Returns all actors contained within a loaded level instance.
	 * Does not require the level instance to be in edit mode.
	 * Returns an empty array if the level instance is not loaded.
	 *
	 * @param LevelInstance to get actors from
	 * @return Array of actors in the level instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Utilities | Level Instance")
	static UE_API TArray<AActor*> LevelInstanceGetActors(ALevelInstance* LevelInstance);
};

#undef UE_API
