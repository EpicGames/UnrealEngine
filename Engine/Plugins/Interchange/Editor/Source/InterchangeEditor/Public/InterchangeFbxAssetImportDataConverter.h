// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeAssetImportData.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeFbxAssetImportDataConverter.generated.h"

#define UE_API INTERCHANGEEDITOR_API

UCLASS(MinimalAPI)
class UInterchangeFbxAssetImportDataConverter : public UInterchangeAssetImportDataConverterBase
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanConvertClass(const UClass* SourceClass, const UClass* DestinationClass) const override;

	UE_API virtual bool ConvertImportData(UObject* Asset, const FString& ToExtension) const override;
	UE_API virtual bool ConvertImportData(const UObject* SourceImportData, const UClass* DestinationClass, UObject** DestinationImportData) const override;
};

#undef UE_API
