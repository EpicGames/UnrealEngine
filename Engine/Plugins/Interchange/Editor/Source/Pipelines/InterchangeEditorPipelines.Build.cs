// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InterchangeEditorPipelines : ModuleRules
	{
		public InterchangeEditorPipelines(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InterchangeCore",
					"InterchangeEngine"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"AssetTools",
					"EngineAssetDefinitions",
					"InputCore",
					"InterchangePipelines",
					"InterchangeNodes",
					"MainFrame",
					"Projects",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"ToolWidgets",
					"UnrealEd"
				}
			);
		}
	}
}
