// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetDefinition_InterchangeBlueprintPipeline.h"
#include "InterchangePipelineFactories.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_InterchangeBlueprintPipeline"

UFactory* UAssetDefinition_InterchangeBlueprintPipeline::GetFactoryForBlueprintType(UBlueprint* InBlueprint) const
{
	UInterchangeBlueprintPipelineBaseFactory* InterchangeBlueprintPipelineBaseFactory = NewObject<UInterchangeBlueprintPipelineBaseFactory>();
	InterchangeBlueprintPipelineBaseFactory->ParentClass = TSubclassOf<UInterchangePipelineBase>(*InBlueprint->GeneratedClass);
	return InterchangeBlueprintPipelineBaseFactory;
}

#undef LOCTEXT_NAMESPACE
