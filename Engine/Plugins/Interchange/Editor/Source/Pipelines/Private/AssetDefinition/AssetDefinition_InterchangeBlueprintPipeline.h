// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once
#include "InterchangeBlueprintPipelineBase.h"
#include "Script/AssetDefinition_Blueprint.h"
#include "AssetDefinition_InterchangeBlueprintPipeline.generated.h"

UCLASS()
class UAssetDefinition_InterchangeBlueprintPipeline : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InterchangeBlueprintPipeline", "Interchange Blueprint Pipeline"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(10, 25, 175); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInterchangeBlueprintPipelineBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Interchange(NSLOCTEXT("InterchangeBlueprintPipeline", "InterchangeCategoryPath", "Interchange"));
		static const auto Categories = { FAssetCategoryPath(Interchange, NSLOCTEXT("AssetDefinition", "AssetDefinition_InterchangeBlueprintPipelineSubMenu", "Interchange Blueprint"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual UFactory* GetFactoryForBlueprintType(UBlueprint* InBlueprint) const override;
	// UAssetDefinition End
};
