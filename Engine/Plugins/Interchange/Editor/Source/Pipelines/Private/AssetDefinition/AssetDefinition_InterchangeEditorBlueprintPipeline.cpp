// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetDefinition_InterchangeEditorBlueprintPipeline.h"
#include "InterchangePipelineFactories.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_InterchangeEditorBlueprintPipeline"

UFactory* UAssetDefinition_InterchangeEditorBlueprintPipeline::GetFactoryForBlueprintType(UBlueprint* InBlueprint) const
{
	UInterchangeEditorBlueprintPipelineBaseFactory* InterchangeEditorBlueprintPipelineBaseFactory = NewObject<UInterchangeEditorBlueprintPipelineBaseFactory>();
	InterchangeEditorBlueprintPipelineBaseFactory->ParentClass = TSubclassOf<UInterchangeEditorPipelineBase>(*InBlueprint->GeneratedClass);
	return InterchangeEditorBlueprintPipelineBaseFactory;
}

#undef LOCTEXT_NAMESPACE
