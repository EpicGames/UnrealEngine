// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once
#include "InterchangeEditorBlueprintPipelineBase.h"
#include "Script/AssetDefinition_Blueprint.h"
#include "AssetDefinition_InterchangeEditorBlueprintPipeline.generated.h"

UCLASS()
class UAssetDefinition_InterchangeEditorBlueprintPipeline : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InterchangeEditorBlueprintPipeline", "Interchange Editor Blueprint Pipeline"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(10, 25, 175); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInterchangeEditorBlueprintPipelineBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Interchange(NSLOCTEXT("InterchangeEditorBlueprintPipeline", "InterchangeCategoryPath", "Interchange"));
		static const auto Categories = { FAssetCategoryPath(Interchange, NSLOCTEXT("AssetDefinition", "AssetDefinition_InterchangeEditorBlueprintPipelineSubMenu", "Interchange Blueprint"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual UFactory* GetFactoryForBlueprintType(UBlueprint* InBlueprint) const override;
	// UAssetDefinition End
};
