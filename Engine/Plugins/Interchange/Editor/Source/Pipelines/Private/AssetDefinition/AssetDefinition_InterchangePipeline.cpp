// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetDefinition_InterchangePipeline.h"
#include "Toolkits/SimpleAssetEditor.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_InterchangePipeline"

EAssetCommandResult UAssetDefinition_InterchangePipeline::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UInterchangePipelineBase* Object : OpenArgs.LoadObjects<UInterchangePipelineBase>())
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
