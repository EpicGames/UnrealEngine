// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once
#include "AssetDefinitionDefault.h"
#include "InterchangePipelineBase.h"
#include "AssetDefinition_InterchangePipeline.generated.h"

UCLASS()
class UAssetDefinition_InterchangePipeline : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InterchangePipeline", "Interchange Pipeline"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(135, 200, 25); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInterchangePipelineBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Interchange(NSLOCTEXT("InterchangePipeline", "InterchangeCategoryPath", "Interchange"));
		static const auto Categories = { FAssetCategoryPath(Interchange, NSLOCTEXT("AssetDefinition", "AssetDefinition_InterchangePipelineSubMenu", "Interchange Pipeline"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
