// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetDefinition_InterchangePythonPipeline.h"
#include "Toolkits/SimpleAssetEditor.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_InterchangePythonPipeline"

EAssetCommandResult UAssetDefinition_InterchangePythonPipeline::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UInterchangePythonPipelineAsset* Object : OpenArgs.LoadObjects<UInterchangePythonPipelineAsset>())
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
