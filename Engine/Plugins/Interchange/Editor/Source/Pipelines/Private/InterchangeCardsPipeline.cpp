// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeCardsPipeline.h"

#include "CoreGlobals.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeCardsPipeline)

void UInterchangeCardsPipeline::SetDisabledFactoryNodes(TArray<UClass*> FactoryNodeClasses)
{
	FactoryNodeClassesToDisabled = FactoryNodeClasses;
}

void UInterchangeCardsPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath)
{
	BaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([this](const FString& NodeUid, UInterchangeFactoryBaseNode* Node)
		{
			if (FactoryNodeClassesToDisabled.Contains(Node->GetClass()))
			{
				Node->SetEnabled(false);
			}
		});
}
