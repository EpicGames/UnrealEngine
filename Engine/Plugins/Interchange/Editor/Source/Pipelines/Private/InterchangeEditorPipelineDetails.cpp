// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeEditorPipelineDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "InterchangePipelineBase.h"
#include "Nodes/InterchangeBaseNode.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "InterchangeEditorPipelineDetails"

FOnGatherConflictAndExtraInfo FInterchangePipelineBaseDetailsCustomization::OnGatherConflictAndExtraInfo;

FInterchangePipelineBaseDetailsCustomization::FInterchangePipelineBaseDetailsCustomization()
{
	InterchangePipeline = nullptr;
	CachedDetailBuilder = nullptr;
}

void FInterchangePipelineBaseDetailsCustomization::RefreshCustomDetail()
{
	if (CachedDetailBuilder)
	{
		ReCollectConflictAndExtraInfo();
		CachedDetailBuilder->ForceRefreshDetails();
	}
}

void FInterchangePipelineBaseDetailsCustomization::LockPropertyHandleRow(const TSharedPtr<IPropertyHandle> PropertyHandle, IDetailPropertyRow& PropertyRow) const
{
	//When we have a locked property we do not show the Reset to default button
	FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateLambda([](TSharedPtr<IPropertyHandle> PropertyHandle)
		{
			return false;
		});
	FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateLambda([](TSharedPtr<IPropertyHandle> PropertyHandle) { return; });
	FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);

	uint32 NumChildren = 0;
	if (PropertyHandle.Get()->GetNumChildren(NumChildren) == FPropertyAccess::Success && NumChildren > 0)
	{
		//Remove the reset to CDO for the locked property
		constexpr bool bShowChildren = false;
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow Row;
		PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
		NameWidget->SetEnabled(false);
		ValueWidget->SetEnabled(false);
		FDetailWidgetRow& WidgetRow = PropertyRow.CustomWidget(bShowChildren)
			.NameContent()
			.MinDesiredWidth(Row.NameWidget.MinWidth)
			.MaxDesiredWidth(Row.NameWidget.MaxWidth)
			[
				NameWidget.ToSharedRef()
			]
		.ValueContent()
			.MinDesiredWidth(Row.ValueWidget.MinWidth)
			.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
			.VAlign(VAlign_Center)
			[
				ValueWidget.ToSharedRef()
			];
		WidgetRow.OverrideResetToDefault(ResetOverride);
	}
	else
	{
		PropertyRow.IsEnabled(false);
		PropertyRow.OverrideResetToDefault(ResetOverride);
	}
}

void FInterchangePipelineBaseDetailsCustomization::AddSubCategory(IDetailLayoutBuilder& DetailBuilder, TMap<FName, TMap<FName, TArray<FInternalPropertyData>>>& SubCategoriesPropertiesPerMainCategory)
{
	for (const TPair<FName, TMap<FName, TArray<FInternalPropertyData>>>& MainCategoryAndSubCategoriesProperties : SubCategoriesPropertiesPerMainCategory)
	{
		const FName& MainCategoryName = MainCategoryAndSubCategoriesProperties.Key;
		const TMap<FName, TArray<FInternalPropertyData>>& SubCategoriesProperties = MainCategoryAndSubCategoriesProperties.Value;
		IDetailCategoryBuilder& MainCategory = DetailBuilder.EditCategory(MainCategoryName);
		//If we found some sub category we can add them to the group
		for (const TPair<FName, TArray<FInternalPropertyData>>& SubCategoryProperties : SubCategoriesProperties)
		{
			const FName& SubCategoryName = SubCategoryProperties.Key;
			const TArray<FInternalPropertyData>& Properties = SubCategoryProperties.Value;
			constexpr bool SubCategoryAdvanced = false;
			IDetailGroup& Group = MainCategory.AddGroup(SubCategoryName, FText::FromName(SubCategoryName), SubCategoryAdvanced);
			for (int32 PropertyIndex = 0; PropertyIndex < Properties.Num(); ++PropertyIndex)
			{
				const FInternalPropertyData& PropertyData = Properties[PropertyIndex];
				const TSharedPtr<IPropertyHandle>& PropertyHandle = PropertyData.PropertyHandle;
				DetailBuilder.HideProperty(PropertyHandle);
				IDetailPropertyRow& PropertyRow = Group.AddPropertyRow(PropertyHandle.ToSharedRef());
				//If needed, make the property read only
				if (PropertyData.bReadOnly)
				{
					LockPropertyHandleRow(PropertyHandle, PropertyRow);
				}
			}
		}
	}
}

TSharedRef<IDetailCustomization> FInterchangePipelineBaseDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangePipelineBaseDetailsCustomization);
}

static TArray<TArray<FInterchangeConflictInfo>> ConflictInfosStack;
void FInterchangePipelineBaseDetailsCustomization::SetConflictsInfo(TArray<FInterchangeConflictInfo>& ConflictInfos)
{
	// Only add info when there are any
	if (ConflictInfos.Num() > 0)
	{
		ConflictInfosStack.Push(ConflictInfos);
	}
}

static TMap<FString, FString> InterchangeAssetExtraInformation;
void FInterchangePipelineBaseDetailsCustomization::SetExtraInformation(const TMap<FString, FString>& InExtraInformation)
{
	InterchangeAssetExtraInformation = InExtraInformation;
}

void FInterchangePipelineBaseDetailsCustomization::SetTextComboBoxWidget(IDetailPropertyRow& PropertyRow, const TSharedPtr<IPropertyHandle>& Handle, const TArray<FString>& PossibleValues)
{
	if (!Handle.IsValid() || !Handle->IsValidHandle() || PossibleValues.Num() < 1)
	{
		//We customize only valid handle and if there is some possible values
		return;
	}

	FProperty* PropertyPtr = Handle->GetProperty();
	if (!PropertyPtr)
	{
		return;
	}
	const bool bIsNameProperty = PropertyPtr->IsA(FNameProperty::StaticClass());
	const bool bIsStringProperty = PropertyPtr->IsA(FStrProperty::StaticClass());
	if (!bIsNameProperty && ! bIsStringProperty)
	{
		//We support only FName and FString property
		return;
	}

	//Create the data in the Map
	FInternalComboBoxData& InternalComboBoxData = ComboBoxDataPerProperty.FindOrAdd(Handle);
	//Generate a combo box row with all the possible value
	InternalComboBoxData.StringSharedOptions.Reset(PossibleValues.Num());
	if (bIsNameProperty)
	{
		InternalComboBoxData.NameOptions.Reset(PossibleValues.Num());
		InternalComboBoxData.StringOptions.Empty();
	}
	else
	{
		InternalComboBoxData.NameOptions.Empty();
		InternalComboBoxData.StringOptions.Reset(PossibleValues.Num());
	}
	
	for (int32 ValueIndex = 0; ValueIndex < PossibleValues.Num(); ++ValueIndex)
	{
		const FString& Value = PossibleValues[ValueIndex];
		InternalComboBoxData.StringSharedOptions.Add(MakeShareable(new FString(Value)));
		if (bIsNameProperty)
		{
			InternalComboBoxData.NameOptions.Add(FName(*Value));
		}
		else
		{
			InternalComboBoxData.StringOptions.Add(Value);
		}
	}

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
	if (PropertyRow.CustomNameWidget())
	{
		//Keep the customize name widget
		NameWidget = PropertyRow.CustomNameWidget()->Widget;
	}

	int32 ComboBoxIndex = INDEX_NONE;
	if (bIsNameProperty)
	{
		FName InitialValue;
		ensure(Handle->GetValue(InitialValue) == FPropertyAccess::Success);
		ComboBoxIndex = InternalComboBoxData.NameOptions.Find(InitialValue);
		if (ComboBoxIndex == INDEX_NONE && InternalComboBoxData.NameOptions.Num() > 0)
		{
			ComboBoxIndex = 0;
			ensure(Handle->SetValue(InternalComboBoxData.NameOptions[ComboBoxIndex]) == FPropertyAccess::Success);
		}
	}
	else
	{
		FString InitialValue;
		ensure(Handle->GetValue(InitialValue) == FPropertyAccess::Success);
		ComboBoxIndex = InternalComboBoxData.StringOptions.Find(InitialValue);
		if (ComboBoxIndex == INDEX_NONE && InternalComboBoxData.StringOptions.Num() > 0)
		{
			ComboBoxIndex = 0;
			ensure(Handle->SetValue(InternalComboBoxData.StringOptions[ComboBoxIndex]) == FPropertyAccess::Success);
		}
	}
	
	check(ComboBoxIndex != INDEX_NONE);
	TWeakPtr<IPropertyHandle> HandlePtr = Handle;

	const bool bShowChildren = true;
	PropertyRow.CustomWidget(bShowChildren)
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		.HAlign(HAlign_Fill)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		.VAlign(VAlign_Center)
		[
			SNew(STextComboBox)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OptionsSource(&InternalComboBoxData.StringSharedOptions)
			.InitiallySelectedItem(InternalComboBoxData.StringSharedOptions[ComboBoxIndex])
			.OnSelectionChanged(this, &FInterchangePipelineBaseDetailsCustomization::OnTextComboBoxChanged, HandlePtr)
		];
}

void FInterchangePipelineBaseDetailsCustomization::OnTextComboBoxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo, TWeakPtr<IPropertyHandle> HandlePtr)
{
	TSharedPtr<IPropertyHandle> Handle = HandlePtr.Pin();
	if (Handle.IsValid() && Handle->IsValidHandle())
	{
		FProperty* PropertyPtr = Handle->GetProperty();
		if (!PropertyPtr)
		{
			return;
		}
		FInternalComboBoxData& InternalComboBoxData = ComboBoxDataPerProperty.FindChecked(Handle);
		const bool bIsNameProperty = PropertyPtr->IsA(FNameProperty::StaticClass());
		const bool bIsStringProperty = PropertyPtr->IsA(FStrProperty::StaticClass());
		if (!bIsNameProperty && !bIsStringProperty)
		{
			//We support only FName and FString property
			return;
		}

		int32 GroupIndex = InternalComboBoxData.StringSharedOptions.Find(NewValue);
		if (ensure(GroupIndex != INDEX_NONE))
		{
			if (bIsNameProperty)
			{
				ensure(Handle->SetValue(InternalComboBoxData.NameOptions[GroupIndex]) == FPropertyAccess::Success);
			}
			else
			{
				ensure(Handle->SetValue(InternalComboBoxData.StringOptions[GroupIndex]) == FPropertyAccess::Success);
			}
		}
	}
}

FReply FInterchangePipelineBaseDetailsCustomization::ShowConflictDialog(FInterchangeConflictInfo ConflictInfo)
{
	ConflictInfo.Pipeline->ShowConflictDialog(ConflictInfo.UniqueId);
	return FReply::Handled();
}

void FInterchangePipelineBaseDetailsCustomization::AddConflictSection()
{
	if (ConflictInfosStack.Num() == 0)
	{
		return;
	}

	TArray<FInterchangeConflictInfo> ConflictInfos = ConflictInfosStack.Pop(EAllowShrinking::No);
	if (ConflictInfos.Num() == 0)
	{
		return;
	}

	static const FName ConflictsName = TEXT("Conflicts");
	static const FText ConflictsText = LOCTEXT("Conflicts_CategoryName", "Conflicts");
	IDetailCategoryBuilder& CategoryBuilder = CachedDetailBuilder->EditCategory(ConflictsName, ConflictsText, ECategoryPriority::Important);

	static const FText Conflict_ButtonText = LOCTEXT("Conflicts_ButtonShow", "Show Conflict");
	static const FSlateBrush* ConflictBrush = FAppStyle::GetBrush("Icons.Error");

	for (const FInterchangeConflictInfo& ConflictInfo : ConflictInfos)
	{
		const FText Conflict_IconTooltip = FText::FromString(ConflictInfo.Description);
		const FText Conflict_NameContent = FText::FromString(ConflictInfo.DisplayName);
		const FText Conflict_ButtonTooltip = FText::Format(LOCTEXT("Conflict_ButtonTooltip", "Show Conflict for {0}"), Conflict_NameContent);

		CategoryBuilder.AddCustomRow(ConflictsText).WholeRowContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.AutoWidth()
				.Padding(2.0f, 2.0f, 5.0f, 2.0f)
				[
					SNew(SImage)
					.ToolTipText(Conflict_IconTooltip)
					.Image(ConflictBrush)
					.ColorAndOpacity(FStyleColors::Warning)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Conflict_NameContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FStyleColors::Warning)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(SButton)
					.ToolTipText(Conflict_ButtonTooltip)
					.OnClicked(this, &FInterchangePipelineBaseDetailsCustomization::ShowConflictDialog, ConflictInfo)
					.Content()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(Conflict_ButtonText)
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				]
			];
	}
}

void FInterchangePipelineBaseDetailsCustomization::AddExtraInformationSection()
{
	if (InterchangeAssetExtraInformation.Num() > 0)
	{
		const FText ExtraInformationCategoryText = LOCTEXT("ExtraInformationCategoryName", "Extra Information");
		const FName ExtraInformationCategoryName = FName(TEXT("Extra Information"));
		IDetailCategoryBuilder& AttributeCategoryBuilder = CachedDetailBuilder->EditCategory(ExtraInformationCategoryName, ExtraInformationCategoryText);
		
		for (const TPair<FString, FString>& Pair : InterchangeAssetExtraInformation)
		{
			FText AttributeName = FText::FromString(Pair.Key);
			FText AttributeValue = FText::FromString(Pair.Value);
			FDetailWidgetRow& CustomRow = AttributeCategoryBuilder.AddCustomRow(AttributeName)
				.NameContent()
				[
					SNew(STextBlock)
						.Text(AttributeName)
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(STextBlock)
						.Text(AttributeValue)
						.Font(IDetailLayoutBuilder::GetDetailFont())
				];
		}

		InterchangeAssetExtraInformation.Reset();
	}
}

void FInterchangePipelineBaseDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedDetailBuilder = &DetailBuilder;
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);

	if (!ensure(EditingObjects.Num() == 1))
	{
		return;
	}

	InterchangePipeline = Cast<UInterchangePipelineBase>(EditingObjects[0].Get());

	if (!ensure(InterchangePipeline.IsValid()))
	{
		return;
	}

	const bool bAllowPropertyStatesEdition = InterchangePipeline->CanEditPropertiesStates();
	const bool bIsReimportContext = InterchangePipeline->IsReimportContext();
	const bool bIsShowEssentials = InterchangePipeline->IsShowEssentials();

	TArray<FName> AllCategoryNames;
	CachedDetailBuilder->GetCategoryNames(AllCategoryNames);
	TMap<FName, TArray<FName>> PropertiesPerCategorys;
	InternalGetPipelineProperties(InterchangePipeline.Get(), AllCategoryNames, PropertiesPerCategorys);
	
	AddConflictSection();
	
	for (const TPair<FName, TArray<FName>>& CategoryAndProperties : PropertiesPerCategorys)
	{
		//Category meta value Subgroup data
		TMap<FString, IDetailGroup*> SubCategoryGroups;

		const FName CategoryName = CategoryAndProperties.Key;
		
		//As prior set displaynames are lost with the call of EditCategory,
		//we make sure that the Category Name is used as is without auto Capital Letter start and automatic spacing.
		IDetailCategoryBuilder& Category = CachedDetailBuilder->EditCategory(CategoryName, FText::FromString(CategoryName.ToString()));
		
		TArray<TSharedRef<IPropertyHandle>> CategoryProperties;

		Category.GetDefaultProperties(CategoryProperties);

		for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
		{
			FProperty* PropertyPtr = PropertyHandle.Get().GetProperty();
			if (!PropertyPtr)
			{
				continue;
			}
			const FName PropertyName = PropertyPtr ? PropertyPtr->GetFName() : NAME_None;
			if (PropertyName == UInterchangePipelineBase::GetPropertiesStatesPropertyName())
			{
				CachedDetailBuilder->HideProperty(PropertyHandle);
				continue;
			}
			// Skip the property not in the List of supported properties
			if (!CategoryAndProperties.Value.Contains(PropertyName))
			{
				continue;
			}

			//If the property UObject owner class is not the same type as the main pipeline class, it mean
			//we deal with a sub pipeline and we must hide property that have the "StandAlonePipelineProperty" meta data
			if(UClass* InterchangePipelinePropertyClass = Cast<UClass>(PropertyPtr->GetOwnerUObject()))
			{
				if (InterchangePipeline->GetClass() != InterchangePipelinePropertyClass && PropertyHandle->GetBoolMetaData(FName("StandAlonePipelineProperty")))
				{
					CachedDetailBuilder->HideProperty(PropertyHandle);
					continue;
				}
			}
			
			const bool PipelineInternalEditionData = PropertyHandle->GetBoolMetaData(FName("PipelineInternalEditionData"));

			//Hide property that are not suppose to show in the import dialog
			if (!bAllowPropertyStatesEdition && PipelineInternalEditionData)
			{
				CachedDetailBuilder->HideProperty(PropertyHandle);
				continue;
			}

			FName PropertyPath = FName(PropertyPtr->GetPathName());
			CachedDetailBuilder->HideProperty(PropertyHandle);

			const FString SubCategoryData = PropertyHandle->GetMetaData(TEXT("SubCategory"));
			IDetailGroup* GroupPtr = nullptr;
			auto GetGroupPtr = [&GroupPtr, &SubCategoryData, &SubCategoryGroups, &Category]()
			{
				if (!SubCategoryData.IsEmpty())
				{
					if (!SubCategoryGroups.Contains(SubCategoryData))
					{
						//Localize sub category
						FText LocalizeSubCategoryName = FText::FromString(SubCategoryData);
						
						if (SubCategoryData.Equals(TEXT("Build")))
						{
							LocalizeSubCategoryName = LOCTEXT("SubCategory_Build", "Build");
						}
						else if (SubCategoryData.Equals(TEXT("Collision")))
						{
							LocalizeSubCategoryName = LOCTEXT("SubCategory_Collision", "Collision");
						}
						else if (SubCategoryData.Equals(TEXT("Actors properties")))
						{
							LocalizeSubCategoryName = LOCTEXT("SubCategory_Actors_properties", "Actors properties");
						}
						else if (SubCategoryData.Equals(TEXT("Reimport Actors")))
						{
							LocalizeSubCategoryName = LOCTEXT("SubCategory_Reimport_Actors", "Reimport Actors");
						}
						else if (SubCategoryData.Equals(TEXT("Reimport Assets")))
						{
							LocalizeSubCategoryName = LOCTEXT("SubCategory_Reimport_Assets", "Reimport Assets");
						}
						SubCategoryGroups.Add(SubCategoryData, &(Category.AddGroup(FName(SubCategoryData), LocalizeSubCategoryName)));
					}
					GroupPtr = SubCategoryGroups.FindChecked(SubCategoryData);
				}
			};

			//This is the import/re-import dialog mode
			if (!bAllowPropertyStatesEdition)
			{
				if (bIsReimportContext && PropertyHandle->GetBoolMetaData(FName("AdjustPipelineAndRefreshDetailOnChange")))
				{
					PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]() {
						InterchangePipeline->AdjustSettingsFromCache();
						RefreshCustomDetail();
					}));
				}

				bool IsLocked = false;
				if (const FInterchangePipelinePropertyStates* PropertyStates = InterchangePipeline->GetPropertyStates(PropertyPath))
				{
					if (!PropertyStates->IsPropertyVisible(bIsReimportContext, bIsShowEssentials))
					{
						continue;
					}
					IsLocked = PropertyStates->IsPropertyLocked();
				}
				GetGroupPtr();
				IDetailPropertyRow& PropertyRow = GroupPtr ? GroupPtr->AddPropertyRow(PropertyHandle) : Category.AddProperty(PropertyHandle);
				//When we use the pipeline in interchange
				if (IsLocked)
				{
					LockPropertyHandleRow(PropertyHandle, PropertyRow);
				}

				//Customize FName and FString properties if the pipeline define some possible value for this property
				if (PropertyPtr->IsA(FNameProperty::StaticClass()) || PropertyPtr->IsA(FStrProperty::StaticClass()))
				{
					TArray<FString> PossibleValues;
					if (InterchangePipeline->GetPropertyPossibleValues(PropertyPath, PossibleValues))
					{
						SetTextComboBoxWidget(PropertyRow, PropertyHandle, PossibleValues);
					}
				}
			}
			//This is the asset editor mode, we allow users to set default value and locks
			else
			{
				constexpr bool bShowChildren = true;
				GetGroupPtr();
				IDetailPropertyRow& PropertyRow = GroupPtr ? GroupPtr->AddPropertyRow(PropertyHandle) : Category.AddProperty(PropertyHandle);
				
				TSharedPtr<SWidget> NameWidget;
				TSharedPtr<SWidget> ValueWidget;
				FDetailWidgetRow Row;
				auto TextColorLambda = [InterchangePipelinePtr = InterchangePipeline, PropertyPath]()
				{
					FSlateColor SlateColor = FStyleColors::Foreground;
					if (InterchangePipelinePtr.IsValid())
					{
						if (const FInterchangePipelinePropertyStates* PropertyStates = InterchangePipelinePtr->GetPropertyStates(PropertyPath))
						{
							if (PropertyStates->IsPropertyLocked())
							{
								SlateColor = FStyleColors::AccentBlue;
							}
						}
					}
					return SlateColor;
				};
				PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
				//When we edit the default value and the locked properties of a pipeline assets
				PropertyRow.CustomWidget(bShowChildren)
				.NameContent()
				.MinDesiredWidth(Row.NameWidget.MinWidth)
				.MaxDesiredWidth(Row.NameWidget.MaxWidth)
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3.0f, 1.0f)
					[
						SNew(SCheckBox)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.CheckedImage(FAppStyle::Get().GetBrush("Icons.Lock"))
						.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Lock"))
						.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Lock"))
						.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
						.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
						.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
						.ToolTipText(LOCTEXT("LockedTooltip", "If true this property will be readonly in the interchange import dialog."))
						.OnCheckStateChanged_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath](ECheckBoxState CheckType)
						{
							if (!ensure(InterchangePipelinePtr.IsValid()))
							{
								return;
							}
							FScopedTransaction ScopedTransaction(LOCTEXT("TransactionTogglePropertyLocked", "Toggle property locked at import"), !GIsTransacting);
							InterchangePipelinePtr->Modify();
							InterchangePipelinePtr->FindOrAddPropertyStates(PropertyPath).SetPropertyLocked(CheckType == ECheckBoxState::Checked);
							InterchangePipelinePtr->PostEditChange();
						})
						.IsChecked_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath]()
						{
							if (!InterchangePipelinePtr.IsValid())
							{
								return ECheckBoxState::Unchecked;
							}
							if (const FInterchangePipelinePropertyStates* PropertyStates = InterchangePipelinePtr->GetPropertyStates(PropertyPath))
							{
								return PropertyStates->IsPropertyLocked() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							}
							return ECheckBoxState::Unchecked;
						})
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Left)
					.Padding(3.0f, 1.0f)
					[
						SNew(SBorder)
						.BorderBackgroundColor(FStyleColors::Transparent)
						.ForegroundColor_Lambda(TextColorLambda)
						[
							NameWidget.ToSharedRef()
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6.0f, 1.0f, 3.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("ResetPreDialogName", "Reset PreDialog"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3.0f, 1.0f)
					[
						SNew(SCheckBox)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.ToolTipText(LOCTEXT("ResetPreDialogTooltip", "If true this property will be reset when displaying the interchange import dialog."))
						.OnCheckStateChanged_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath](ECheckBoxState CheckType)
						{
							if (!ensure(InterchangePipelinePtr.IsValid()))
							{
								return;
							}
							FScopedTransaction ScopedTransaction(LOCTEXT("TransactionResetPreDialogToggle", "Toggle property reset pre-dialog."), !GIsTransacting);
							InterchangePipelinePtr->Modify();
							InterchangePipelinePtr->FindOrAddPropertyStates(PropertyPath).SetPropertyPreDialogReset((CheckType == ECheckBoxState::Checked));
							InterchangePipelinePtr->PostEditChange();
						})
						.IsChecked_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath]()
						{
							if (!InterchangePipelinePtr.IsValid())
							{
								return ECheckBoxState::Unchecked;
							}
							if (const FInterchangePipelinePropertyStates* PropertyStates = InterchangePipelinePtr->GetPropertyStates(PropertyPath))
							{
								return PropertyStates->IsPropertyPreDialogReset() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							}
							return ECheckBoxState::Unchecked;
						})
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6.0f, 1.0f, 3.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("ShowWhenShowEssentialsText", "Essential Layout"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3.0f, 1.0f)
					[
						SNew(SCheckBox)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.CheckedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.ToolTipText(LOCTEXT("VisibleTooltipShowEssentials", "If true this property will be visible when displaying the interchange import dialog with basic layout."))
						.OnCheckStateChanged_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath](ECheckBoxState CheckType)
						{
							if (!ensure(InterchangePipelinePtr.IsValid()))
							{
								return;
							}
							FScopedTransaction ScopedTransaction(LOCTEXT("TransactionvisibilityPropertiesToggleImport", "Toggle property visibility at import"), !GIsTransacting);
							InterchangePipelinePtr->Modify();
							InterchangePipelinePtr->FindOrAddPropertyStates(PropertyPath).SetPropertyShowEssentialsVisibility((CheckType != ECheckBoxState::Checked));
							InterchangePipelinePtr->PostEditChange();
						})
						.IsChecked_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath]()
						{
							if (!InterchangePipelinePtr.IsValid())
							{
								return ECheckBoxState::Unchecked;
							}
							if (const FInterchangePipelinePropertyStates* PropertyStates = InterchangePipelinePtr->GetPropertyStates(PropertyPath))
							{
								return PropertyStates->IsPropertyVisibleInShowEssentials() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
							}
							return ECheckBoxState::Unchecked;
						})
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6.0f, 1.0f, 3.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("HiddenAtImportText", "Import"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3.0f, 1.0f)
					[
						SNew(SCheckBox)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.CheckedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.ToolTipText(LOCTEXT("VisibleTooltipImport", "If true this property will be visible when displaying the interchange import dialog."))
						.OnCheckStateChanged_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath](ECheckBoxState CheckType)
						{
							if (!ensure(InterchangePipelinePtr.IsValid()))
							{
								return;
							}
							FScopedTransaction ScopedTransaction(LOCTEXT("TransactionvisibilityPropertiesToggleImport", "Toggle property visibility at import"), !GIsTransacting);
							InterchangePipelinePtr->Modify();
							InterchangePipelinePtr->FindOrAddPropertyStates(PropertyPath).SetPropertyImportVisibility((CheckType != ECheckBoxState::Checked));
							InterchangePipelinePtr->PostEditChange();
						})
						.IsChecked_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath]()
						{
							constexpr bool bIsReimportContextLocal = false;
							constexpr bool bIsShowEssentialsLocal = false;
							if (!InterchangePipelinePtr.IsValid())
							{
								return ECheckBoxState::Unchecked;
							}
							if (const FInterchangePipelinePropertyStates* PropertyStates = InterchangePipelinePtr->GetPropertyStates(PropertyPath))
							{
								return PropertyStates->IsPropertyVisible(bIsReimportContextLocal, bIsShowEssentialsLocal) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
							}
							return ECheckBoxState::Unchecked;
						})
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6.0f, 1.0f, 3.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("HiddenAtReimportText", "Reimport"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3.0f, 1.0f)
					[
						SNew(SCheckBox)
						.Visibility(PipelineInternalEditionData ? EVisibility::Collapsed : EVisibility::Visible)
						.CheckedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
						.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
						.ToolTipText(LOCTEXT("VisibleTooltipReimport", "If true this property will be visible when displaying the interchange reimport dialog."))
						.OnCheckStateChanged_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath](ECheckBoxState CheckType)
						{
							if (!ensure(InterchangePipelinePtr.IsValid()))
							{
								return;
							}
							FScopedTransaction ScopedTransaction(LOCTEXT("TransactionvisibilityPropertiesToggleReimport", "Toggle property visibility at reimport"), !GIsTransacting);
							InterchangePipelinePtr->Modify();
							InterchangePipelinePtr->FindOrAddPropertyStates(PropertyPath).SetPropertyReimportVisibility((CheckType != ECheckBoxState::Checked));
							InterchangePipelinePtr->PostEditChange();
						})
						.IsChecked_Lambda([InterchangePipelinePtr = InterchangePipeline, PropertyPath]()
						{
							constexpr bool bIsReimportContextLocal = true;
							constexpr bool bIsShowEssentialsLocal = false;
							if (!InterchangePipelinePtr.IsValid())
							{
								return ECheckBoxState::Unchecked;
							}
							if (const FInterchangePipelinePropertyStates* PropertyStates = InterchangePipelinePtr->GetPropertyStates(PropertyPath))
							{
								return PropertyStates->IsPropertyVisible(bIsReimportContextLocal, bIsShowEssentialsLocal) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
							}
							return ECheckBoxState::Unchecked;
						})
					]
				]
				.ValueContent()
				.MinDesiredWidth(Row.ValueWidget.MinWidth)
				.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
				.VAlign(VAlign_Center)
				[
					ValueWidget.ToSharedRef()
				];

				//Customize FName and FString properties if the pipeline define some possible value for this property
				if (PropertyPtr->IsA(FNameProperty::StaticClass()) || PropertyPtr->IsA(FStrProperty::StaticClass()))
				{
					TArray<FString> PossibleValues;
					if (InterchangePipeline->GetPropertyPossibleValues(PropertyPath, PossibleValues))
					{
						SetTextComboBoxWidget(PropertyRow, PropertyHandle, PossibleValues);
					}
				}
			}
		}
	}

	if (!bIsShowEssentials)
	{
		AddExtraInformationSection();
	}
}

void FInterchangePipelineBaseDetailsCustomization::ReCollectConflictAndExtraInfo()
{
	TArray<FInterchangeConflictInfo> ConflictInfo;
	TMap<FString, FString> ExtraInfo;
	OnGatherConflictAndExtraInfo.ExecuteIfBound(ConflictInfo, ExtraInfo);
	SetConflictsInfo(ConflictInfo);
	SetExtraInformation(ExtraInfo);
}

FInterchangeBaseNodeDetailsCustomization::FInterchangeBaseNodeDetailsCustomization()
{
	InterchangeBaseNode = nullptr;
	CachedDetailBuilder = nullptr;
}

FInterchangeBaseNodeDetailsCustomization::~FInterchangeBaseNodeDetailsCustomization()
{
}

void FInterchangeBaseNodeDetailsCustomization::RefreshCustomDetail()
{
	if (CachedDetailBuilder)
	{
		CachedDetailBuilder->ForceRefreshDetails();
	}
}

void FInterchangePipelineBaseDetailsCustomization::InternalGetPipelineProperties(const UInterchangePipelineBase* Pipeline, const TArray<FName>& AllCategoryNames, TMap<FName, TArray<FName>>& PropertiesPerCategorys) const
{
	const UClass* PipelineClass = Pipeline->GetClass();
	const FName CategoryKey("Category");
	TArray<UInterchangePipelineBase*> SubPipelines;
	for (FProperty* Property = PipelineClass->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		//Do not load a transient property
		if (Property->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}
		FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		if (UInterchangePipelineBase* SubPipeline = SubObject ? Cast<UInterchangePipelineBase>(SubObject->GetObjectPropertyValue_InContainer(Pipeline)) : nullptr)
		{
			SubPipelines.Add(SubPipeline);
		}
		else if (const FString* PropertyCategoryString = Property->FindMetaData(CategoryKey))
		{
			FName PropertyCategoryName(*PropertyCategoryString);
			if (AllCategoryNames.Contains(PropertyCategoryName))
			{
				PropertiesPerCategorys.FindOrAdd(PropertyCategoryName).AddUnique(Property->GetFName());
			}
		}
	}
	//Get all sub pipeline object properties
	for (UInterchangePipelineBase* SubPipeline : SubPipelines)
	{
		InternalGetPipelineProperties(SubPipeline, AllCategoryNames, PropertiesPerCategorys);
	}
}


TSharedRef<IDetailCustomization> FInterchangeBaseNodeDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangeBaseNodeDetailsCustomization);
}

void FInterchangeBaseNodeDetailsCustomization::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	CachedDetailBuilder = &DetailBuilder;
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	InterchangeBaseNode = Cast<UInterchangeBaseNode>(EditingObjects[0].Get());

	if (!ensure(InterchangeBaseNode))
	{
		return;
	}

	//Add a row to describe the node type and class
	{
		const FText NodeInformationCategoryText = LOCTEXT("NodeInformationCategoryName", "Node Information");
		const FName NodeInformationCategoryName = FName(TEXT("Node Information"));
		IDetailCategoryBuilder& AttributeCategoryBuilder = DetailBuilder.EditCategory(NodeInformationCategoryName, NodeInformationCategoryText);
		
		FText NodeTypeValueText = FText::GetEmpty();
		FText NodeClassValueText = FText::GetEmpty();
		FText ObjectClassNameValueText = FText::GetEmpty();

		NodeClassValueText = FText::FromString(InterchangeBaseNode->GetClass()->GetName());

		if (UInterchangeFactoryBaseNode* FactoryNode = Cast<UInterchangeFactoryBaseNode>(InterchangeBaseNode))
		{
			const FString ClassName = FactoryNode->GetObjectClass() 
										? FactoryNode->GetObjectClass()->GetName() 
										: "None - Unprocessed";

			NodeTypeValueText = LOCTEXT("NodeFactoryInformation_FactoryNodeValueText", "Factory Node");
			ObjectClassNameValueText = FText::FromString(ClassName);
		}
		else
		{
			NodeTypeValueText = LOCTEXT("NodeFactoryInformation_TranslatedNodeValueText", "Translated Node");
		}

		auto AddNodeInformationRow = [&AttributeCategoryBuilder, &NodeInformationCategoryText](const FText& Name, const FText& Value)
			{
				AttributeCategoryBuilder.AddCustomRow(NodeInformationCategoryText)
					.FilterString(Name)
					.NameContent()
					[
						SNew(STextBlock)
							.Text(Name)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						SNew(STextBlock)
							.Text(Value)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					];
			};

		const FText NodeTypeNameText = LOCTEXT("NodeFactoryInformationNodeTypeNameText", "Node Type");
		AddNodeInformationRow(NodeTypeNameText, NodeTypeValueText);

		const FText NodeClassNameText = LOCTEXT("NodeFactoryInformationNodeClassNameText", "Node Class");
		AddNodeInformationRow(NodeClassNameText, NodeClassValueText);

		if (!ObjectClassNameValueText.IsEmpty())
		{
			const FText ObjectClassNameNameText = LOCTEXT("NodeFactoryInformationObjectClassNameNameText", "Object Class Name");
			AddNodeInformationRow(ObjectClassNameNameText, ObjectClassNameValueText);
		}
	}

	TArray< UE::Interchange::FAttributeKey> AttributeKeys;
	InterchangeBaseNode->GetAttributeKeys(AttributeKeys);

	TMap<FString, TArray< UE::Interchange::FAttributeKey>> AttributesPerCategory;
	for (UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
	{
		if (InterchangeBaseNode->ShouldHideAttribute(AttributeKey))
		{
			//Skip attribute we should hide
			continue;
		}
		const FString CategoryName = InterchangeBaseNode->GetAttributeCategory(AttributeKey);
		TArray< UE::Interchange::FAttributeKey>& CategoryAttributeKeys = AttributesPerCategory.FindOrAdd(CategoryName);
		CategoryAttributeKeys.Add(AttributeKey);
	}

	//Add all categories
	for (TPair<FString, TArray< UE::Interchange::FAttributeKey>>& CategoryAttributesPair : AttributesPerCategory)
	{
		FName CategoryName = FName(*CategoryAttributesPair.Key);
		IDetailCategoryBuilder& AttributeCategoryBuilder = DetailBuilder.EditCategory(CategoryName, FText::GetEmpty());
		for (UE::Interchange::FAttributeKey& AttributeKey : CategoryAttributesPair.Value)
		{
			AddAttributeRow(AttributeKey, AttributeCategoryBuilder);
		}
	}
}

void FInterchangeBaseNodeDetailsCustomization::AddAttributeRow(UE::Interchange::FAttributeKey& AttributeKey, IDetailCategoryBuilder& AttributeCategory)
{
	if (!ensure(InterchangeBaseNode))
	{
		return;
	}
	const UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
	switch (AttributeType)
	{
		case UE::Interchange::EAttributeTypes::Bool:
		{
			BuildBoolValueContent(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Double:
		{
			BuildNumberValueContent < double > (AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Float:
		{
			BuildNumberValueContent < float >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Int8:
		{
			BuildNumberValueContent < int8 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Int16:
		{
			BuildNumberValueContent < int16 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Int32:
		{
			BuildNumberValueContent < int32 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Int64:
		{
			BuildNumberValueContent < int64 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::UInt8:
		{
			BuildNumberValueContent < uint8 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::UInt16:
		{
			BuildNumberValueContent < uint16 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::UInt32:
		{
			BuildNumberValueContent < uint32 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::UInt64:
		{
			BuildNumberValueContent < uint64 >(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::String:
		{
			BuildStringValueContent<FString>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Name:
		{
			BuildStringValueContent<FName>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Transform3f:
		{
			BuildTransformValueContent<FTransform3f, FVector3f, FQuat4f, float>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Transform3d:
		{
			BuildTransformValueContent<FTransform3d, FVector3d, FQuat4d, double>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Box3f:
		{
			BuildBoxValueContent<FBox3f, FVector3f, float>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Box3d:
		{
			BuildBoxValueContent<FBox3d, FVector3d, double>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Vector3f:
		{
			BuildVectorValueContent<FVector3f, float>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Vector3d:
		{
			BuildVectorValueContent<FVector3d, double>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::SoftObjectPath:
		{
			BuildStringValueContent<FSoftObjectPath>(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::Color:
		{
			BuildColorValueContent(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::LinearColor:
		{
			BuildLinearColorValueContent(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::InterchangeLODMeshData:
		{
			BuildStringInterchangeMeshLODDataValueContent(AttributeCategory, AttributeKey);
		}
		break;

		case UE::Interchange::EAttributeTypes::InterchangeLODMeshDataArray:
		{
			BuildStringInterchangeMeshLODDataArrayValueContent(AttributeCategory, AttributeKey);
		}
		break;

		default:
		{
			FText AttributeName = FText::FromString(InterchangeBaseNode->GetKeyDisplayName(AttributeKey));
			AttributeCategory.AddCustomRow(AttributeName)
			.NameContent()
			[
				CreateNameWidget(AttributeKey)
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("UnsupportedCustomizationType", "Attribute Type Not Supported"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];
		}
	}
}

void FInterchangeBaseNodeDetailsCustomization::BuildBoolValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
	check(AttributeType == UE::Interchange::EAttributeTypes::Bool);
	UE::Interchange::FAttributeStorage::TAttributeHandle<bool> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<bool>(AttributeKey);
	if (!AttributeHandle.IsValid())
	{
		CreateInvalidHandleRow(AttributeCategory, AttributeKey);
		return;
	}
	
	const FText AttributeName = FText::FromString(InterchangeBaseNode->GetKeyDisplayName(AttributeKey));
	FDetailWidgetRow& CustomRow = AttributeCategory.AddCustomRow(AttributeName)
	.NameContent()
	[
		CreateNameWidget(AttributeKey)
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			[
				SNew(SCheckBox)
				.IsEnabled(InterchangeBaseNode->UserInterfaceContext == EInterchangeNodeUserInterfaceContext::None)
				.OnCheckStateChanged_Lambda([this, AttributeKey](ECheckBoxState CheckType)
				{
					const bool IsChecked = CheckType == ECheckBoxState::Checked;
					UE::Interchange::FAttributeStorage::TAttributeHandle<bool> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<bool>(AttributeKey);
					if (AttributeHandle.IsValid())
					{
						AttributeHandle.Set(IsChecked);
					}
				})
				.IsChecked_Lambda([this, AttributeKey]()
				{
					bool IsChecked = false;
					UE::Interchange::FAttributeStorage::TAttributeHandle<bool> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<bool>(AttributeKey);
					if (AttributeHandle.IsValid())
					{
						AttributeHandle.Get(IsChecked);
					}
					return IsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			]
		]
	];
}

template<typename NumericType>
void FInterchangeBaseNodeDetailsCustomization::BuildNumberValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::FAttributeStorage::TAttributeHandle<NumericType> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<NumericType>(AttributeKey);
	if (!AttributeHandle.IsValid())
	{
		CreateInvalidHandleRow(AttributeCategory, AttributeKey);
		return;
	}

	auto GetValue = [](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->NumericType
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<NumericType> AttributeHandle = BaseNode->GetAttributeHandle<NumericType>(Key);
		//Prevent returning uninitialize value by setting it to 0
		NumericType Value = static_cast<NumericType>(0);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(Value);
		}
		return Value;
	};

	auto SetValue = [](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, NumericType Value, UE::Interchange::FAttributeKey& Key)
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<NumericType> AttributeHandle = BaseNode->GetAttributeHandle<NumericType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Set(Value);
		}
    };

	const FText AttributeName = FText::FromString(InterchangeBaseNode->GetKeyDisplayName(AttributeKey));
	AttributeCategory.AddCustomRow(AttributeName)
	.NameContent()
	[
		CreateNameWidget(AttributeKey)
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			MakeNumericWidget<NumericType>(0, GetValue, SetValue, AttributeKey)
		]
	];
}

template<typename StringType>
void FInterchangeBaseNodeDetailsCustomization::BuildStringValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::FAttributeStorage::TAttributeHandle<StringType> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<StringType>(AttributeKey);
	if (!AttributeHandle.IsValid())
	{
		CreateInvalidHandleRow(AttributeCategory, AttributeKey);
		return;
	}

	const FText AttributeName = FText::FromString(InterchangeBaseNode->GetKeyDisplayName(AttributeKey));
	FDetailWidgetRow& CustomRow = AttributeCategory.AddCustomRow(AttributeName)	
    .NameContent()
    [
        CreateNameWidget(AttributeKey)
    ]
    .ValueContent()
    [
        SNew(SHorizontalBox)
        + SHorizontalBox::Slot()
        .AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
        [
            SNew(STextBlock)
            .Text_Lambda([this, AttributeKey]()->FText
            {
				UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
				FText ReturnText;
				if (AttributeType == UE::Interchange::EAttributeTypes::String)
				{
					UE::Interchange::FAttributeStorage::TAttributeHandle<FString> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FString>(AttributeKey);
					FString Value;
					if (AttributeHandle.IsValid())
					{
						AttributeHandle.Get(Value);
					}
					ReturnText = FText::FromString(Value);
				}
				else if (AttributeType == UE::Interchange::EAttributeTypes::Name)
				{
					UE::Interchange::FAttributeStorage::TAttributeHandle<FName> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FName>(AttributeKey);
					FName Value;
					if (AttributeHandle.IsValid())
					{
						AttributeHandle.Get(Value);
					}
					ReturnText = FText::FromName(Value);
				}
				else if (AttributeType == UE::Interchange::EAttributeTypes::SoftObjectPath)
				{
					UE::Interchange::FAttributeStorage::TAttributeHandle<FSoftObjectPath> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FSoftObjectPath>(AttributeKey);
					FSoftObjectPath Value;
					if (AttributeHandle.IsValid())
					{
						AttributeHandle.Get(Value);
					}
					ReturnText = FText::FromString(Value.ToString());
				}
                return ReturnText;
            })
            .Font(IDetailLayoutBuilder::GetDetailFont())
        ]
    ];
}
template<typename TransformnType, typename VectorType, typename QuatType, typename NumericType>
void FInterchangeBaseNodeDetailsCustomization::BuildTransformValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);

	{
		const UE::Interchange::FAttributeStorage::TAttributeHandle<TransformnType> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<TransformnType>(AttributeKey);
		if (!AttributeHandle.IsValid())
		{
			CreateInvalidHandleRow(AttributeCategory, AttributeKey);
			return;
		}
	}

	const bool bAdvancedProperty = false;
	const FString GroupName = InterchangeBaseNode->GetKeyDisplayName(AttributeKey);
	IDetailGroup& Group = AttributeCategory.AddGroup(FName(*GroupName), FText::FromString(GroupName), bAdvancedProperty);
	FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow();
	GroupHeaderRow.NameContent().Widget = SNew(SBox)
	[
		CreateNameWidget(AttributeKey)
	];

	auto GetRotationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->QuatType
	{
		TransformnType TransformValue;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<TransformnType> AttributeHandle = BaseNode->GetAttributeHandle<TransformnType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(TransformValue);
		}
		return TransformValue.GetRotation();
	};

	auto SetRotationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const QuatType& Value)
	{
		TransformnType TransformValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<TransformnType> AttributeHandle = BaseNode->GetAttributeHandle<TransformnType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(TransformValue);
			TransformValue.SetRotation(Value);
			AttributeHandle.Set(TransformValue);
		}
	};

	auto GetTranslationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->VectorType
	{
		TransformnType TransformValue;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<TransformnType> AttributeHandle = BaseNode->GetAttributeHandle<TransformnType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(TransformValue);
		}
		return TransformValue.GetTranslation();
	};

	auto SetTranslationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const VectorType& Value)
	{
		TransformnType TransformValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<TransformnType> AttributeHandle = BaseNode->GetAttributeHandle<TransformnType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(TransformValue);
			TransformValue.SetTranslation(Value);
			AttributeHandle.Set(TransformValue);
		}
	};

	auto GetScale3DValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->VectorType
	{
		TransformnType TransformValue;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<TransformnType> AttributeHandle = BaseNode->GetAttributeHandle<TransformnType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(TransformValue);
		}
		return TransformValue.GetScale3D();
	};

	auto SetScale3DValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const VectorType& Value)
	{
		TransformnType TransformValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<TransformnType> AttributeHandle = BaseNode->GetAttributeHandle<TransformnType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(TransformValue);
			TransformValue.SetScale3D(Value);
			AttributeHandle.Set(TransformValue);
		}
	};

	const FString TranslationName = TEXT("Translation");
	Group.AddWidgetRow()
    .NameContent()
    [
        CreateSimpleNameWidget(TranslationName)
    ]
    .ValueContent()
    [
        CreateVectorWidget<VectorType, NumericType>(GetTranslationValue, SetTranslationValue, AttributeKey)
    ];

	const FString RotationName = TEXT("Rotation");
	Group.AddWidgetRow()
	.NameContent()
	[
		CreateSimpleNameWidget(RotationName)
	]
	.ValueContent()
	[
		CreateQuaternionWidget<QuatType, NumericType>(GetRotationValue, SetRotationValue, AttributeKey)
	];

	const FString Scale3DName = TEXT("Scale3D");
	Group.AddWidgetRow()
    .NameContent()
    [
        CreateSimpleNameWidget(Scale3DName)
    ]
    .ValueContent()
    [
        CreateVectorWidget<VectorType, NumericType>(GetScale3DValue, SetScale3DValue, AttributeKey)
    ];
}

void FInterchangeBaseNodeDetailsCustomization::BuildStringInterchangeMeshLODDataArrayValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);

	const UE::Interchange::FAttributeStorage::TAttributeHandle<TArray<FInterchangeLODMeshData>> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<TArray<FInterchangeLODMeshData>>(AttributeKey);
	if (!AttributeHandle.IsValid())
	{
		CreateInvalidHandleRow(AttributeCategory, AttributeKey);
		return;
	}
	TArray<FInterchangeLODMeshData> LODMeshDataArrayValue;
	if (AttributeHandle.IsValid())
	{
		AttributeHandle.Get(LODMeshDataArrayValue);
	}

	const FString KeyDisplayName = InterchangeBaseNode->GetKeyDisplayName(AttributeKey);

	for (int32 LODMeshCounter = 0; LODMeshCounter < LODMeshDataArrayValue.Num(); LODMeshCounter++)
	{
		const FText AttributeNameMeshUid = FText::FromString(KeyDisplayName + TEXT("[") + FString::FromInt(LODMeshCounter) + TEXT("]") + TEXT(".MeshUid"));
		const FText AttributeTooltipMeshUid = FText::FromString(TEXT("Attribute Type: ") + UE::Interchange::AttributeTypeToString(AttributeType) + TEXT(" (MeshUid)"));

		const FText AttributeNameTransform = FText::FromString(KeyDisplayName + TEXT("[") + FString::FromInt(LODMeshCounter) + TEXT("]") + TEXT(".Transform"));
		const FText AttributeTooltipTransform = FText::FromString(TEXT("Attribute Type: ") + UE::Interchange::AttributeTypeToString(AttributeType) + TEXT(" (Transform)"));

		//MeshUid
		{
			FDetailWidgetRow& CustomRow = AttributeCategory.AddCustomRow(AttributeNameMeshUid)
				.NameContent()
				[
					SNew(STextBlock)
						.Text(AttributeNameMeshUid)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.ToolTipText(AttributeTooltipMeshUid)
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
								.Text(FText::FromString(LODMeshDataArrayValue[LODMeshCounter].MeshUid))
						]
				];
		}

		//Transform
		{
			auto AddNumericWidget = [](double X, double Y, double Z, double W = 0, bool bUseW = false) -> TSharedRef<SHorizontalBox>
				{
					TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SNumericEntryBox<double>)
								.IsEnabled(false)
								.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
								.Value(X)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.AllowSpin(false)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SNumericEntryBox<double>)
								.IsEnabled(false)
								.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
								.Value(Y)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.AllowSpin(false)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SNumericEntryBox<double>)
								.IsEnabled(false)
								.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
								.Value(Z)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.AllowSpin(false)
						];

						if (bUseW)
						{
							HorizontalBox->AddSlot()
								.AutoWidth()
								[
									SNew(SNumericEntryBox<double>)
										.IsEnabled(false)
										.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
										.Value(W)
										.Font(IDetailLayoutBuilder::GetDetailFont())
										.AllowSpin(false)
								];
						}

					return HorizontalBox;
				};


			const bool bAdvancedProperty = false;
			IDetailGroup& Group = AttributeCategory.AddGroup(FName(*AttributeNameTransform.ToString()), AttributeNameTransform, bAdvancedProperty);
			FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow();
			GroupHeaderRow.NameContent().Widget = SNew(SBox)
				[
					SNew(STextBlock)
						.Text(AttributeNameTransform)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.ToolTipText(AttributeTooltipTransform)
				];

			FQuat4d Rotation = LODMeshDataArrayValue[LODMeshCounter].Transform.GetRotation();
			FVector3d Translation = LODMeshDataArrayValue[LODMeshCounter].Transform.GetTranslation();
			FVector3d Scale3D = LODMeshDataArrayValue[LODMeshCounter].Transform.GetScale3D();

			const FString TranslationName = TEXT("Translation");
			Group.AddWidgetRow()
				.NameContent()
				[
					CreateSimpleNameWidget(TranslationName)
				]
				.ValueContent()
				[
					AddNumericWidget(Translation.X, Translation.Y, Translation.Z)
				];

			const FString RotationName = TEXT("Rotation");
			Group.AddWidgetRow()
				.NameContent()
				[
					CreateSimpleNameWidget(RotationName)
				]
				.ValueContent()
				[
					AddNumericWidget(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W, true)
				];

			const FString Scale3DName = TEXT("Scale3D");
			Group.AddWidgetRow()
				.NameContent()
				[
					CreateSimpleNameWidget(Scale3DName)
				]
				.ValueContent()
				[
					AddNumericWidget(Scale3D.X, Scale3D.Y, Scale3D.Z)
				];
		}
	}
}

void FInterchangeBaseNodeDetailsCustomization::BuildStringInterchangeMeshLODDataValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);

	{
		const UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FInterchangeLODMeshData>(AttributeKey);
		if (!AttributeHandle.IsValid())
		{
			CreateInvalidHandleRow(AttributeCategory, AttributeKey);
			return;
		}
	}

	const FString KeyDisplayName = InterchangeBaseNode->GetKeyDisplayName(AttributeKey);

	const FText AttributeNameMeshUid = FText::FromString(KeyDisplayName + (TEXT(".MeshUid")));
	const FText AttributeNameSceneNodeName = FText::FromString(KeyDisplayName + (TEXT(".SceneNodeName")));
	const FText AttributeNameTransform = FText::FromString(KeyDisplayName + (TEXT(".Transform")));
	const FText AttributeNameGeometricTransform = FText::FromString(KeyDisplayName + (TEXT(".GeometricTransform")));

	const FString AttributeTooltipStringMeshUid = TEXT("Attribute Type: ") + UE::Interchange::AttributeTypeToString(AttributeType) + TEXT(" (MeshUid)");
	const FString AttributeTooltipStringSceneNodeName = TEXT("Attribute Type: ") + UE::Interchange::AttributeTypeToString(AttributeType) + TEXT(" (SceneNodeName)");
	const FString AttributeTooltipStringTransform = TEXT("Attribute Type: ") + UE::Interchange::AttributeTypeToString(AttributeType) + TEXT(" (Transform)");
	const FString AttributeTooltipStringGeometricTransform = TEXT("Attribute Type: ") + UE::Interchange::AttributeTypeToString(AttributeType) + TEXT(" (GeometricTransform)");

	const FText AttributeTooltipMeshUid = FText::FromString(AttributeTooltipStringMeshUid);
	const FText AttributeTooltipSceneNodeName = FText::FromString(AttributeTooltipStringSceneNodeName);
	const FText AttributeTooltipTransform = FText::FromString(AttributeTooltipStringTransform);
	const FText AttributeTooltipGeometricTransform = FText::FromString(AttributeTooltipStringGeometricTransform);

	//MeshUid
	{
		const FText AttributeName = FText::FromString(TEXT("MeshUid"));
		FDetailWidgetRow& CustomRow = AttributeCategory.AddCustomRow(AttributeName)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(AttributeNameMeshUid)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(AttributeTooltipMeshUid)
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text_Lambda([this, AttributeKey]()->FText
								{
									UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
									FText ReturnText;
									if (AttributeType == UE::Interchange::EAttributeTypes::InterchangeLODMeshData)
									{
										UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FInterchangeLODMeshData>(AttributeKey);
										FInterchangeLODMeshData Value;
										if (AttributeHandle.IsValid())
										{
											AttributeHandle.Get(Value);
										}
										ReturnText = FText::FromString(Value.MeshUid);
									}
									return ReturnText;
								})
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}

	//SceneNodeName
	{
		const FText AttributeName = FText::FromString(TEXT("SceneNodeName"));
		FDetailWidgetRow& CustomRow = AttributeCategory.AddCustomRow(AttributeName)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(AttributeNameSceneNodeName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(AttributeTooltipSceneNodeName)
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text_Lambda([this, AttributeKey]()->FText
								{
									UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
									FText ReturnText;
									if (AttributeType == UE::Interchange::EAttributeTypes::InterchangeLODMeshData)
									{
										UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FInterchangeLODMeshData>(AttributeKey);
										FInterchangeLODMeshData Value;
										if (AttributeHandle.IsValid())
										{
											AttributeHandle.Get(Value);
										}
										ReturnText = FText::FromString(Value.SceneNodeName);
									}
									return ReturnText;
								})
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}
	
	//Transform
	{
		const bool bAdvancedProperty = false;
		const FString GroupName = InterchangeBaseNode->GetKeyDisplayName(AttributeKey) + TEXT(".Transform");
		IDetailGroup& Group = AttributeCategory.AddGroup(FName(*GroupName), FText::FromString(GroupName), bAdvancedProperty);
		FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow();
		GroupHeaderRow.NameContent().Widget = SNew(SBox)
			[
				SNew(STextBlock)
					.Text(AttributeNameTransform)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(AttributeTooltipTransform)
			];

		auto GetRotationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->FQuat4d
			{
				FInterchangeLODMeshData NodeUidTransformsPair;
				const UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(NodeUidTransformsPair);
				}
				return NodeUidTransformsPair.Transform.GetRotation();
			};

		auto SetRotationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const FQuat4d& Value)
			{
				FInterchangeLODMeshData NodeUidTransformsPair;
				UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(NodeUidTransformsPair);
					NodeUidTransformsPair.Transform.SetRotation(Value);
					AttributeHandle.Set(NodeUidTransformsPair);
				}
			};

		auto GetTranslationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->FVector3d
			{
				FInterchangeLODMeshData NodeUidTransformsPair;
				const UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(NodeUidTransformsPair);
				}
				return NodeUidTransformsPair.Transform.GetTranslation();
			};

		auto SetTranslationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const FVector3d& Value)
			{
				FInterchangeLODMeshData NodeUidTransformsPair;
				UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(NodeUidTransformsPair);
					NodeUidTransformsPair.Transform.SetTranslation(Value);
					AttributeHandle.Set(NodeUidTransformsPair);
				}
			};

		auto GetScale3DValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->FVector3d
			{
				FInterchangeLODMeshData NodeUidTransformsPair;
				const UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(NodeUidTransformsPair);
				}
				return NodeUidTransformsPair.Transform.GetScale3D();
			};

		auto SetScale3DValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const FVector3d& Value)
			{
				FInterchangeLODMeshData NodeUidTransformsPair;
				UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(NodeUidTransformsPair);
					NodeUidTransformsPair.Transform.SetScale3D(Value);
					AttributeHandle.Set(NodeUidTransformsPair);
				}
			};

		const FString TranslationName = TEXT("Translation");
		Group.AddWidgetRow()
			.NameContent()
			[
				CreateSimpleNameWidget(TranslationName)
			]
			.ValueContent()
			[
				CreateVectorWidget<FVector3d, double>(GetTranslationValue, SetTranslationValue, AttributeKey)
			];

		const FString RotationName = TEXT("Rotation");
		Group.AddWidgetRow()
			.NameContent()
			[
				CreateSimpleNameWidget(RotationName)
			]
			.ValueContent()
			[
				CreateQuaternionWidget<FQuat4d, double>(GetRotationValue, SetRotationValue, AttributeKey)
			];

		const FString Scale3DName = TEXT("Scale3D");
		Group.AddWidgetRow()
			.NameContent()
			[
				CreateSimpleNameWidget(Scale3DName)
			]
			.ValueContent()
			[
				CreateVectorWidget<FVector3d, double>(GetScale3DValue, SetScale3DValue, AttributeKey)
			];
	}

	//GeometricTransform
	{
		const bool bAdvancedProperty = false;
		const FString GroupName = InterchangeBaseNode->GetKeyDisplayName(AttributeKey) + TEXT(".GeometricTransform");
		IDetailGroup& Group = AttributeCategory.AddGroup(FName(*GroupName), FText::FromString(GroupName), bAdvancedProperty);
		FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow();
		GroupHeaderRow.NameContent().Widget = SNew(SBox)
			[
				SNew(STextBlock)
					.Text(AttributeNameGeometricTransform)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(AttributeTooltipGeometricTransform)
			];

		auto GetRotationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->FQuat4d
			{
				FInterchangeLODMeshData LODMeshData;
				const UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(LODMeshData);
				}
				return LODMeshData.GeometricTransform.GetRotation();
			};

		auto SetRotationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const FQuat4d& Value)
			{
				FInterchangeLODMeshData LODMeshData;
				UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(LODMeshData);
					LODMeshData.GeometricTransform.SetRotation(Value);
					AttributeHandle.Set(LODMeshData);
				}
			};

		auto GetTranslationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->FVector3d
			{
				FInterchangeLODMeshData LODMeshData;
				const UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(LODMeshData);
				}
				return LODMeshData.GeometricTransform.GetTranslation();
			};

		auto SetTranslationValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const FVector3d& Value)
			{
				FInterchangeLODMeshData LODMeshData;
				UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(LODMeshData);
					LODMeshData.GeometricTransform.SetTranslation(Value);
					AttributeHandle.Set(LODMeshData);
				}
			};

		auto GetScale3DValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->FVector3d
			{
				FInterchangeLODMeshData LODMeshData;
				const UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(LODMeshData);
				}
				return LODMeshData.GeometricTransform.GetScale3D();
			};

		auto SetScale3DValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const FVector3d& Value)
			{
				FInterchangeLODMeshData LODMeshData;
				UE::Interchange::FAttributeStorage::TAttributeHandle<FInterchangeLODMeshData> AttributeHandle = BaseNode->GetAttributeHandle<FInterchangeLODMeshData>(Key);
				if (AttributeHandle.IsValid())
				{
					AttributeHandle.Get(LODMeshData);
					LODMeshData.GeometricTransform.SetScale3D(Value);
					AttributeHandle.Set(LODMeshData);
				}
			};

		const FString TranslationName = TEXT("Translation");
		Group.AddWidgetRow()
			.NameContent()
			[
				CreateSimpleNameWidget(TranslationName)
			]
			.ValueContent()
			[
				CreateVectorWidget<FVector3d, double>(GetTranslationValue, SetTranslationValue, AttributeKey)
			];

		const FString RotationName = TEXT("Rotation");
		Group.AddWidgetRow()
			.NameContent()
			[
				CreateSimpleNameWidget(RotationName)
			]
			.ValueContent()
			[
				CreateQuaternionWidget<FQuat4d, double>(GetRotationValue, SetRotationValue, AttributeKey)
			];

		const FString Scale3DName = TEXT("Scale3D");
		Group.AddWidgetRow()
			.NameContent()
			[
				CreateSimpleNameWidget(Scale3DName)
			]
			.ValueContent()
			[
				CreateVectorWidget<FVector3d, double>(GetScale3DValue, SetScale3DValue, AttributeKey)
			];
	}
}

void FInterchangeBaseNodeDetailsCustomization::BuildColorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
	if (AttributeType == UE::Interchange::EAttributeTypes::Color)
	{
		const UE::Interchange::FAttributeStorage::TAttributeHandle<FColor> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FColor>(AttributeKey);
		if (!AttributeHandle.IsValid())
		{
			CreateInvalidHandleRow(AttributeCategory, AttributeKey);
			return;
		}
	}
	else
	{
		ensure(AttributeType == UE::Interchange::EAttributeTypes::Color);
		CreateInvalidHandleRow(AttributeCategory, AttributeKey);
		return;
	}
	InternalBuildColorValueContent<FColor, uint8>(AttributeCategory, AttributeKey, 255);
}

void FInterchangeBaseNodeDetailsCustomization::BuildLinearColorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
	if (AttributeType == UE::Interchange::EAttributeTypes::LinearColor)
	{
		const UE::Interchange::FAttributeStorage::TAttributeHandle<FLinearColor> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<FLinearColor>(AttributeKey);
		if (!AttributeHandle.IsValid())
		{
			CreateInvalidHandleRow(AttributeCategory, AttributeKey);
			return;
		}
	}
	else
	{
		ensure(AttributeType == UE::Interchange::EAttributeTypes::LinearColor);
		CreateInvalidHandleRow(AttributeCategory, AttributeKey);
		return;
	}
	InternalBuildColorValueContent<FLinearColor, float>(AttributeCategory, AttributeKey, 1.0f);
}

template<typename AttributeType, typename NumericType>
void FInterchangeBaseNodeDetailsCustomization::InternalBuildColorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey, NumericType DefaultTypeValue)
{
	const bool bAdvancedProperty = false;
	const FString GroupName = InterchangeBaseNode->GetKeyDisplayName(AttributeKey);
	IDetailGroup& Group = AttributeCategory.AddGroup(FName(*GroupName), FText::FromString(GroupName), bAdvancedProperty);
	FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow();
	GroupHeaderRow.NameContent().Widget = SNew(SBox)
	[
		CreateNameWidget(AttributeKey)
	];

	auto GetRValue = [DefaultTypeValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->NumericType
	{
		AttributeType Value;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(Value);
		}
		else
		{
			//Error return white color
			return DefaultTypeValue;
		}
		return Value.R;
	};

	auto SetRValue = [](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, const NumericType& Value, UE::Interchange::FAttributeKey& Key)
	{
		AttributeType ColorValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(ColorValue);
			ColorValue.R = Value;
			AttributeHandle.Set(ColorValue);
		}
	};

	auto GetGValue = [DefaultTypeValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->NumericType
	{
		AttributeType Value;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(Value);
		}
		else
		{
			//Error return white color
			return DefaultTypeValue;
		}
		return Value.G;
	};

	auto SetGValue = [](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, const NumericType& Value, UE::Interchange::FAttributeKey& Key)
	{
		AttributeType ColorValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(ColorValue);
			ColorValue.G = Value;
			AttributeHandle.Set(ColorValue);
		}
	};

	auto GetBValue = [DefaultTypeValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->NumericType
	{
		AttributeType Value;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(Value);
		}
		else
		{
			//Error return white color
			return DefaultTypeValue;
		}
		return Value.B;
	};

	auto SetBValue = [](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, const NumericType& Value, UE::Interchange::FAttributeKey& Key)
	{
		AttributeType ColorValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(ColorValue);
			ColorValue.B = Value;
			AttributeHandle.Set(ColorValue);
		}
	};

	auto GetAValue = [DefaultTypeValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->NumericType
	{
		AttributeType Value;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(Value);
		}
		else
		{
			//Error return white color
			return DefaultTypeValue;
		}
		return Value.A;
	};

	auto SetAValue = [](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, const NumericType& Value, UE::Interchange::FAttributeKey& Key)
	{
		AttributeType ColorValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> AttributeHandle = BaseNode->GetAttributeHandle<AttributeType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(ColorValue);
			ColorValue.A = Value;
			AttributeHandle.Set(ColorValue);
		}
	};

	const FString RName = TEXT("Red");
	Group.AddWidgetRow()
    .NameContent()
    [
        CreateSimpleNameWidget(RName)
    ]
    .ValueContent()
    [
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			MakeNumericWidget<NumericType>(0, GetRValue, SetRValue, AttributeKey)
		]
    ];
	const FString GName = TEXT("Green");
	Group.AddWidgetRow()
    .NameContent()
    [
        CreateSimpleNameWidget(GName)
    ]
    .ValueContent()
    [
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			MakeNumericWidget<NumericType>(0, GetGValue, SetGValue, AttributeKey)
		]
    ];

	const FString BName = TEXT("Blue");
	Group.AddWidgetRow()
    .NameContent()
    [
        CreateSimpleNameWidget(BName)
    ]
    .ValueContent()
    [
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			MakeNumericWidget<NumericType>(0, GetBValue, SetBValue, AttributeKey)
		]
    ];

	const FString AName = TEXT("Alpha");
	Group.AddWidgetRow()
    .NameContent()
    [
        CreateSimpleNameWidget(AName)
    ]
    .ValueContent()
    [
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			MakeNumericWidget<NumericType>(0, GetAValue, SetAValue, AttributeKey)
		]
    ];
}

template<typename BoxType, typename VectorType, typename NumericType>
void FInterchangeBaseNodeDetailsCustomization::BuildBoxValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
	{
		const UE::Interchange::FAttributeStorage::TAttributeHandle<BoxType> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<BoxType>(AttributeKey);
		if (!AttributeHandle.IsValid())
		{
			CreateInvalidHandleRow(AttributeCategory, AttributeKey);
			return;
		}
	}

	const bool bAdvancedProperty = false;
	const FString GroupName = InterchangeBaseNode->GetKeyDisplayName(AttributeKey);
	IDetailGroup& Group = AttributeCategory.AddGroup(FName(*GroupName), FText::FromString(GroupName), bAdvancedProperty);
	FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow();
	GroupHeaderRow.NameContent().Widget = SNew(SBox)
	[
		CreateNameWidget(AttributeKey)
	];

	auto GetMinimumValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->VectorType
	{
		BoxType BoxValue;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<BoxType> AttributeHandle = BaseNode->GetAttributeHandle<BoxType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(BoxValue);
		}
		return BoxValue.Min;
	};

	auto SetMinimumValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const VectorType& Value)
	{
		BoxType BoxValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<BoxType> AttributeHandle = BaseNode->GetAttributeHandle<BoxType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(BoxValue);
			BoxValue.Min = Value;
			AttributeHandle.Set(BoxValue);
		}
	};

	auto GetMaximumValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->VectorType
	{
		BoxType BoxValue;
		const UE::Interchange::FAttributeStorage::TAttributeHandle<BoxType> AttributeHandle = BaseNode->GetAttributeHandle<BoxType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(BoxValue);
		}
		return BoxValue.Max;
	};

	auto SetMaximumValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const VectorType& Value)
	{
		BoxType BoxValue;
		UE::Interchange::FAttributeStorage::TAttributeHandle<BoxType> AttributeHandle = BaseNode->GetAttributeHandle<BoxType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(BoxValue);
			BoxValue.Max = Value;
			AttributeHandle.Set(BoxValue);
		}
	};

	const FString MinimumVectorName = TEXT("Minimum");
	Group.AddWidgetRow()
	.NameContent()
	[
		CreateSimpleNameWidget(MinimumVectorName)
	]
	.ValueContent()
	[
		CreateVectorWidget<VectorType, NumericType>(GetMinimumValue, SetMinimumValue, AttributeKey)
	];

	const FString MaximumVectorName = TEXT("Maximum");
	Group.AddWidgetRow()
	.NameContent()
	[
		CreateSimpleNameWidget(MaximumVectorName)
	]
	.ValueContent()
	[
		CreateVectorWidget<VectorType, NumericType>(GetMaximumValue, SetMaximumValue, AttributeKey)
	];
}

template<typename VectorType, typename NumericType>
void FInterchangeBaseNodeDetailsCustomization::BuildVectorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey)
{
	UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);

	{
		const UE::Interchange::FAttributeStorage::TAttributeHandle<VectorType> AttributeHandle = InterchangeBaseNode->GetAttributeHandle<VectorType>(AttributeKey);
		if (!AttributeHandle.IsValid())
		{
			CreateInvalidHandleRow(AttributeCategory, AttributeKey);
			return;
		}
	}

	auto GetValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->VectorType
	{
		VectorType VectorValue = {};
		const UE::Interchange::FAttributeStorage::TAttributeHandle<VectorType> AttributeHandle = BaseNode->GetAttributeHandle<VectorType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Get(VectorValue);
		}
		return VectorValue;
	};

	auto SetValue = [](UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key, const VectorType& Value)
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<VectorType> AttributeHandle = BaseNode->GetAttributeHandle<VectorType>(Key);
		if (AttributeHandle.IsValid())
		{
			AttributeHandle.Set(Value);
		}
	};

	const bool bAdvancedProperty = false;
	FDetailWidgetRow& VectorRow = AttributeCategory.AddCustomRow(FText::FromString(AttributeKey.ToString()), bAdvancedProperty);
	VectorRow.NameContent()
	[
		CreateNameWidget(AttributeKey)
	]
	.ValueContent()
	[
		CreateVectorWidget<VectorType, NumericType>(GetValue, SetValue, AttributeKey)
	];
}

void FInterchangeBaseNodeDetailsCustomization::CreateInvalidHandleRow(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey) const
{
	const FString InvalidAttributeHandle = TEXT("Invalid Attribute Handle!");
	const FText AttributeName = FText::FromString(InterchangeBaseNode->GetKeyDisplayName(AttributeKey));
	AttributeCategory.AddCustomRow(AttributeName)
	.NameContent()
	[
		CreateNameWidget(AttributeKey)
	]
	.ValueContent()
	[
		CreateSimpleNameWidget(InvalidAttributeHandle)
	];
}

TSharedRef<SWidget> FInterchangeBaseNodeDetailsCustomization::CreateNameWidget(UE::Interchange::FAttributeKey& AttributeKey) const
{
	const UE::Interchange::EAttributeTypes AttributeType = InterchangeBaseNode->GetAttributeType(AttributeKey);
	const FText AttributeName = FText::FromString(InterchangeBaseNode->GetKeyDisplayName(AttributeKey));
	const FString AttributeTooltipString = TEXT("Attribute Type: ") + UE::Interchange::AttributeTypeToString(AttributeType);
	const FText AttributeTooltip = FText::FromString(AttributeTooltipString);
	return SNew(STextBlock)
		.Text(AttributeName)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.ToolTipText(AttributeTooltip);
		
}

TSharedRef<SWidget> FInterchangeBaseNodeDetailsCustomization::CreateSimpleNameWidget(const FString& SimpleName) const
{
	const FText SimpleNameText = FText::FromString(SimpleName);
	return SNew(STextBlock)
		.Text(SimpleNameText)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

template<typename VectorType, typename NumericWidgetType, typename FunctorGet, typename FunctorSet>
TSharedRef<SWidget> FInterchangeBaseNodeDetailsCustomization::CreateVectorWidget(FunctorGet GetValue, FunctorSet SetValue, UE::Interchange::FAttributeKey& AttributeKey)
{
	auto GetComponentValue = [&GetValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->NumericWidgetType
	{
		VectorType Value = GetValue(BaseNode, Key);
		return Value[ComponentIndex];
	};

	auto SetComponentValue = [&GetValue, &SetValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, NumericWidgetType ComponentValue, UE::Interchange::FAttributeKey& Key)
	{
		VectorType Value = GetValue(BaseNode, Key);
		Value[ComponentIndex] = ComponentValue;
		SetValue(BaseNode, Key, Value);
    };
	
	//Create a horizontal layout with the 3 floats components
	return SNew(SHorizontalBox)
        + SHorizontalBox::Slot()
        .AutoWidth()
        [
			MakeNumericWidget<NumericWidgetType>(0, GetComponentValue, SetComponentValue, AttributeKey)
        ]
		+ SHorizontalBox::Slot()
	    .AutoWidth()
	    [
	        MakeNumericWidget<NumericWidgetType>(1, GetComponentValue, SetComponentValue, AttributeKey)
	    ]
		+ SHorizontalBox::Slot()
	    .AutoWidth()
	    [
	        MakeNumericWidget<NumericWidgetType>(2, GetComponentValue, SetComponentValue, AttributeKey)
	    ];
}

template<typename QuatType, typename NumericWidgetType, typename FunctorGet, typename FunctorSet>
TSharedRef<SWidget> FInterchangeBaseNodeDetailsCustomization::CreateQuaternionWidget(FunctorGet GetValue, FunctorSet SetValue, UE::Interchange::FAttributeKey& AttributeKey)
{
	auto GetComponentValue = [&GetValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, UE::Interchange::FAttributeKey& Key)->NumericWidgetType
	{
		QuatType Value = GetValue(BaseNode, Key);
		if(ComponentIndex == 0)
		{
			return Value.X;
		}
		else if(ComponentIndex == 1)
		{
			return Value.Y;
		}
		else if(ComponentIndex == 2)
		{
			return Value.Z;
		}
		else if(ComponentIndex == 3)
		{
			return Value.W;
		}

		//Ensure
		ensure(ComponentIndex >= 0 && ComponentIndex < 4);
		return 0.0f;	
	};

	auto SetComponentValue = [&GetValue, &SetValue](int32 ComponentIndex, UInterchangeBaseNode* BaseNode, NumericWidgetType ComponentValue, UE::Interchange::FAttributeKey& Key)
	{
		QuatType Value = GetValue(BaseNode, Key);
		if(ComponentIndex == 0)
		{
			Value.X = ComponentValue;
		}
		else if(ComponentIndex == 1)
		{
			Value.Y = ComponentValue;
		}
		else if(ComponentIndex == 2)
		{
			Value.Z = ComponentValue;
		}
		else if(ComponentIndex == 3)
		{
			Value.W = ComponentValue;
		}
		//Ensure
		if(ensure(ComponentIndex >= 0 && ComponentIndex < 4))
		{
			SetValue(BaseNode, Key, Value);
		}
	};
	
	//Create a horizontal layout with the 4 floats components
	return SNew(SHorizontalBox)
        + SHorizontalBox::Slot()
        .AutoWidth()
        [
            MakeNumericWidget<NumericWidgetType>(0, GetComponentValue, SetComponentValue, AttributeKey)
        ]
        + SHorizontalBox::Slot()
        .AutoWidth()
        [
            MakeNumericWidget<NumericWidgetType>(1, GetComponentValue, SetComponentValue, AttributeKey)
        ]
        + SHorizontalBox::Slot()
        .AutoWidth()
        [
            MakeNumericWidget<NumericWidgetType>(2, GetComponentValue, SetComponentValue, AttributeKey)
        ]
		+ SHorizontalBox::Slot()
	    .AutoWidth()
	    [
	        MakeNumericWidget<NumericWidgetType>(3, GetComponentValue, SetComponentValue, AttributeKey)
	    ];
}

template<typename NumericType, typename FunctorGet, typename FunctorSet>
TSharedRef<SWidget> FInterchangeBaseNodeDetailsCustomization::MakeNumericWidget(int32 ComponentIndex, FunctorGet GetValue, FunctorSet SetValue, UE::Interchange::FAttributeKey& AttributeKey)
{
	//The 3 Lambda functions need to call other lambda function having AttributeKey reference (&AttributeKey).
	//The per value capture prevent us the AttributeKey by reference 
	//We copy the key only one time here and pass a reference of this copy

	auto SetValueCommittedLambda = [this, &SetValue, ComponentIndex, AttributeKey](const NumericType Value, ETextCommit::Type CommitType)
	{
		UE::Interchange::FAttributeKey Key = AttributeKey;
		SetValue(ComponentIndex, InterchangeBaseNode, Value, Key);
	};
	auto SetValueChangedLambda = [this, &SetValue, ComponentIndex, AttributeKey](const NumericType Value)
	{
		UE::Interchange::FAttributeKey Key = AttributeKey;
		SetValue(ComponentIndex, InterchangeBaseNode, Value, Key);
	};
	
	return
		SNew(SNumericEntryBox<NumericType>)
			.IsEnabled(InterchangeBaseNode->UserInterfaceContext == EInterchangeNodeUserInterfaceContext::None)
			.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
			.Value_Lambda([this, &GetValue, ComponentIndex, AttributeKey]()
			{
				UE::Interchange::FAttributeKey Key = AttributeKey;
				return GetValue(ComponentIndex, InterchangeBaseNode, Key);
			})
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OnValueCommitted_Lambda(SetValueCommittedLambda)
			.OnValueChanged_Lambda(SetValueChangedLambda)
			.AllowSpin(false);
}

#undef LOCTEXT_NAMESPACE
