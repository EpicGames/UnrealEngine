// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Delegates/DelegateCombinations.h"

class SWidget;

class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IPropertyHandle;
class UInterchangeBaseNode;
class UInterchangePipelineBase;
struct FInterchangeConflictInfo;

namespace UE
{
	namespace Interchange
	{
		struct FAttributeKey;
	}
}

using FInterchangeConflictInfoArray = TArray<FInterchangeConflictInfo>;
using FInterchangeExtraInfoMap = TMap<FString, FString>;
DECLARE_DELEGATE_TwoParams(FOnGatherConflictAndExtraInfo, FInterchangeConflictInfoArray&, FInterchangeExtraInfoMap&);

class FInterchangePipelineBaseDetailsCustomization : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// Temporary: Used to pass data to the next instance created.
	static void SetConflictsInfo(TArray<FInterchangeConflictInfo>& ConflictInfos);

	// Temporary: Used to pass ExtraInformation to the next instance created.
	static void SetExtraInformation(const TMap<FString, FString>& InExtraInformation);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */


	static void ReCollectConflictAndExtraInfo();
	static FOnGatherConflictAndExtraInfo OnGatherConflictAndExtraInfo;

private:
	struct FInternalPropertyData
	{
		TSharedPtr<IPropertyHandle> PropertyHandle;
		bool bReadOnly = false;
	};
	void RefreshCustomDetail();
	void LockPropertyHandleRow(const TSharedPtr<IPropertyHandle> PropertyHandle, IDetailPropertyRow& PropertyRow) const;
	void AddSubCategory(IDetailLayoutBuilder& DetailBuilder, TMap<FName, TMap<FName, TArray<FInternalPropertyData>>>& SubCategoriesPropertiesPerMainCategory);
	void InternalGetPipelineProperties(const UInterchangePipelineBase* Pipeline, const TArray<FName>& AllCategoryNames, TMap<FName, TArray<FName>>& PropertiesPerCategorys) const;
	void AddConflictSection();
	void AddExtraInformationSection();
	FReply ShowConflictDialog(FInterchangeConflictInfo ConflictInfo);

	void SetTextComboBoxWidget(IDetailPropertyRow& PropertyRow, const TSharedPtr<IPropertyHandle>& Handle, const TArray<FString>& PossibleValues);
	void OnTextComboBoxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo, TWeakPtr<IPropertyHandle> HandlePtr);
	struct FInternalComboBoxData
	{
		TArray<TSharedPtr<FString>> StringSharedOptions;
		TArray<FString> StringOptions;
		TArray<FName> NameOptions;
	};
	TMap<TSharedPtr<IPropertyHandle>, FInternalComboBoxData> ComboBoxDataPerProperty;

	/** Use MakeInstance to create an instance of this class */
	FInterchangePipelineBaseDetailsCustomization();

	TWeakObjectPtr<UInterchangePipelineBase> InterchangePipeline;
	IDetailLayoutBuilder* CachedDetailBuilder;	// The detail builder for this customisation
};

class FInterchangeBaseNodeDetailsCustomization : public IDetailCustomization
{
public:
	~FInterchangeBaseNodeDetailsCustomization();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	/** End IDetailCustomization interface */

private:
	void RefreshCustomDetail();
	void AddAttributeRow(UE::Interchange::FAttributeKey& AttributeKey, IDetailCategoryBuilder& AttributeCategory);

	/*******************************************************************
	 * AttributeType Build functions Begin
	 */
	
	 //Bool: Add a row to the category containing a check box (read/write)
	void BuildBoolValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);
	
	//Number (float, double, signed/unsigned integer): Add one row to the category containing a numeric field (read/write)
	template<typename NumericType>
	void BuildNumberValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	//String (FString, FName): Add a row to the category containing text block (read only)
	template<typename StringType>
	void BuildStringValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	//FTransform: Add a group to the category that contain the Translation(FVector), Rotation(FQuat) and scale3D(FVector)
	//Any numeric field are read/write
	template<typename TransformnType, typename VectorType, typename QuatType, typename NumericType>
	void BuildTransformValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	void BuildStringInterchangeMeshLODDataValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);
	void BuildStringInterchangeMeshLODDataArrayValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	//FColor: Add a row to the category that contain the RGBA value
	//Any numeric field are read/write
	void BuildColorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	//FLinearColor: Add a row to the category that contain the RGBA value
	//Any numeric field are read/write
	void BuildLinearColorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	template<typename AttributeType, typename NumericType>
	void InternalBuildColorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey, NumericType DefaultTypeValue);

	//FBox: Add a group to the category that contain the Minimum corner position(FVector), and the maximum corner position(FVector)
	//Any numeric field are read/write
	template<typename BoxType, typename VectorType, typename NumericType>
	void BuildBoxValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	//FVector: Add a row to the category that display a FVector.
	//Any numeric field are read/write
	template<typename VectorType, typename NumericType>
	void BuildVectorValueContent(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey);

	/**
	 * AttributeType Build functions End
	 ******************************************************************/

	/*
	 * Add a row to the specified category which let the user that a attribute handle is invalid 
	 */
	void CreateInvalidHandleRow(IDetailCategoryBuilder& AttributeCategory, UE::Interchange::FAttributeKey& AttributeKey) const;


	/*******************************************************************
	 * Basic type widgets creation Begin
	 */

	TSharedRef<SWidget> CreateNameWidget(UE::Interchange::FAttributeKey& AttributeKey) const;

	TSharedRef<SWidget> CreateSimpleNameWidget(const FString& SimpleName) const;

	//FVector Create a 3 digit widget (X, Y, Z)
	template<typename VectorType, typename NumericWidgetType, typename FunctorGet, typename FunctorSet>
	TSharedRef<SWidget> CreateVectorWidget(FunctorGet GetValue, FunctorSet SetValue, UE::Interchange::FAttributeKey& AttributeKey);

	//FQuat Create a 4 digit widget (X, Y, Z, W)
	template<typename QuatType, typename NumericWidgetType, typename FunctorGet, typename FunctorSet>
    TSharedRef<SWidget> CreateQuaternionWidget(FunctorGet GetValue, FunctorSet SetValue, UE::Interchange::FAttributeKey& AttributeKey);

	//Create one numeric widget, support (float, double, signed/unsigned integer)
	template<typename NumericType, typename FunctorGet, typename FunctorSet>
	TSharedRef<SWidget> MakeNumericWidget(int32 ComponentIndex, FunctorGet GetValue, FunctorSet SetValue, UE::Interchange::FAttributeKey& AttributeKey);

	/*
	 * Basic type widgets creation End
	 *******************************************************************/

	/** Use MakeInstance to create an instance of this class */
	FInterchangeBaseNodeDetailsCustomization();

	UInterchangeBaseNode* InterchangeBaseNode;
	IDetailLayoutBuilder* CachedDetailBuilder;	// The detail builder for this customisation
};
