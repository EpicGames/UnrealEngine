// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPluginManager.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

/**
 * Implements the visual style of the text asset editor UI.
 */
class FInterchangeEditorPipelineStyle final
	: public FSlateStyleSet
{
public:
	/** Default constructor. */
	FInterchangeEditorPipelineStyle()
		: FSlateStyleSet("InterchangeEditorPipelineStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);

		const FString BaseDir = IPluginManager::Get().FindPlugin("InterchangeEditor")->GetBaseDir();
		SetContentRoot(BaseDir / TEXT("Content"));

		//GraphInspectorIcon icons
		FSlateImageBrush* LodBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_Lod_Icon_16"), TEXT(".png")), Icon16x16);
		Set("SceneGraphIcon.LodGroup", LodBrush16);
		FSlateImageBrush* JointBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_Joint_Icon_16"), TEXT(".png")), Icon16x16);
		Set("SceneGraphIcon.Joint", JointBrush16);
		FSlateImageBrush* StaticMeshBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_StaticMesh_Icon_16"), TEXT(".png")), Icon16x16);
		Set("MeshIcon.Static", StaticMeshBrush16);
		FSlateImageBrush* SkeletalMeshBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_SkeletalMesh_Icon_16"), TEXT(".png")), Icon16x16);
		Set("MeshIcon.Skinned", SkeletalMeshBrush16);

		//PipelineConfigurationIcon icons
		FSlateImageBrush* PipelineBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_Pipeline_Icon_16"), TEXT(".png")), Icon16x16);
		Set("PipelineConfigurationIcon.Pipeline", PipelineBrush16);
		FSlateImageBrush* PipelineStackBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_PipelineStack_Icon_16"), TEXT(".png")), Icon16x16);
		Set("PipelineConfigurationIcon.PipelineStack", PipelineStackBrush16);
		FSlateImageBrush* PipelineStackDefaultBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_PipelineStackDefault_Icon_16"), TEXT(".png")), Icon16x16);
		Set("PipelineConfigurationIcon.PipelineStackDefault", PipelineStackDefaultBrush16);
		FSlateImageBrush* PipelineStackPresetBrush20 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_PipelineStackPreset_Icon_20"), TEXT(".png")), Icon20x20);
		Set("PipelineConfigurationIcon.PipelineStackPreset", PipelineStackPresetBrush20);
		FSlateImageBrush* TranslatorSettingsBrush16 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/Interchange_TranslatorSettings_16"), TEXT(".png")), Icon16x16);
		Set("PipelineConfigurationIcon.TranslatorSettings", TranslatorSettingsBrush16);

		FSlateImageBrush* SidePanelRightBrush20 = new FSlateImageBrush(RootToContentDir(TEXT("Resources/SidePanelRight"), TEXT(".png")), Icon20x20);
		Set("PipelineConfigurationIcon.SidePanelRight", SidePanelRightBrush20);

		Set("ImportSource.Dropdown.Border", new FSlateRoundedBoxBrush(FStyleColors::Dropdown.GetSpecifiedColor(), 4.f));
		Set("AssetCardList.Background.Border", new FSlateRoundedBoxBrush(FStyleColors::Recessed.GetSpecifiedColor(), 4.f));
		Set("AssetCard.Header.Border", new FSlateRoundedBoxBrush(FStyleColors::Dropdown.GetSpecifiedColor(), 4.f));
		Set("AssetCard.Body.Border", new FSlateRoundedBoxBrush(FStyleColors::Panel.GetSpecifiedColor(), 0.f));

		const FButtonStyle ResetSelectedPipelineButtonStyle = FButtonStyle()
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
			.SetDisabled(FSlateNoResource())
			.SetNormalForeground(FStyleColors::Foreground)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Foreground)
			.SetNormalPadding(FMargin(8.f, 4.f, 8.f, 4.f))
			.SetPressedPadding(FMargin(8.f, 5.5f, 8.f, 2.5f));

		Set("ButtonStyle.ResetSelectedPipeline", ResetSelectedPipelineButtonStyle);

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	/** Destructor. */
	~FInterchangeEditorPipelineStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};
