// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeEditorPipelinesModule.h"

#include "AssetTypeCategories.h"
#include "InterchangeEditorPipelineDetails.h"
#include "InterchangeEditorPipelineStyle.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialXPipeline.h"
#include "InterchangeMaterialXPipelineCustomizations.h"
#include "InterchangeglTFPipeline.h"
#include "InterchangeGLTFPipelineCustomizations.h"
#include "InterchangePipelineBase.h"
#include "InterchangePipelineFactories.h"
#include "Misc/CoreDelegates.h"
#include "Nodes/InterchangeBaseNode.h"
#include "PropertyEditorModule.h"



#define LOCTEXT_NAMESPACE "InterchangeEditorPipelines"

class FInterchangeEditorPipelinesModule : public IInterchangeEditorPipelinesModule
{
public:
	FInterchangeEditorPipelinesModule()
		: InterchangeAssetCategory(EAssetTypeCategories::Misc)
	{
	}
private:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//Called when we start the module
	void AcquireResources();
	
	//Can be called multiple time
	void ReleaseResources();

	TSharedRef<FPropertySection> RegisterPropertySection(FPropertyEditorModule& PropertyModule, FName ClassName, FName SectionName, FText DisplayName);
	void RegisterPropertySectionMappings();
	void UnregisterPropertySectionMappings();

	EAssetTypeCategories::Type InterchangeAssetCategory;

	TArray<FName> ClassesToUnregisterOnShutdown;

	/** Pointer to the style set to use for the UI. */
	TSharedPtr<ISlateStyle> InterchangeEditorPipelineStyle = nullptr;

	TMultiMap<FName, FName> RegisteredPropertySections;
	TArray<FName> PropertiesTypesToUnregisterOnShutdown;
};

IMPLEMENT_MODULE(FInterchangeEditorPipelinesModule, InterchangeEditorPipelines)

void FInterchangeEditorPipelinesModule::StartupModule()
{
	AcquireResources();
}

void FInterchangeEditorPipelinesModule::ShutdownModule()
{}

void FInterchangeEditorPipelinesModule::AcquireResources()
{
	auto RegisterItems = [this]()
	{
		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

		RegisterPropertySectionMappings();
	};

	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}

	ClassesToUnregisterOnShutdown.Reset();
	// Register details customizations for animation controller nodes
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	ClassesToUnregisterOnShutdown.Add(UInterchangeBaseNode::StaticClass()->GetFName());
	PropertyEditorModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeBaseNodeDetailsCustomization::MakeInstance));
	
	ClassesToUnregisterOnShutdown.Add(UInterchangePipelineBase::StaticClass()->GetFName());
	PropertyEditorModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangePipelineBaseDetailsCustomization::MakeInstance));

	ClassesToUnregisterOnShutdown.Add(UInterchangeMaterialXPipeline::StaticClass()->GetFName());
	PropertyEditorModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeMaterialXPipelineCustomization::MakeInstance));

	ClassesToUnregisterOnShutdown.Add(UMaterialXPipelineSettings::StaticClass()->GetFName());
	PropertyEditorModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeMaterialXPipelineSettingsCustomization::MakeInstance));

	ClassesToUnregisterOnShutdown.Add(UInterchangeGLTFPipeline::StaticClass()->GetFName());
	PropertyEditorModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeGLTFPipelineCustomization::MakeInstance));

	ClassesToUnregisterOnShutdown.Add(UGLTFPipelineSettings::StaticClass()->GetFName());
	PropertyEditorModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeGLTFPipelineSettingsCustomization::MakeInstance));

	if (!InterchangeEditorPipelineStyle.IsValid())
	{
		InterchangeEditorPipelineStyle = MakeShared<FInterchangeEditorPipelineStyle>();
	}

	// Register the InterchangeImportTestPlan asset
	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	IAssetTools& AssetTools = AssetToolsModule.Get();

	InterchangeAssetCategory = AssetTools.RegisterAdvancedAssetCategory(FName(TEXT("Interchange")), NSLOCTEXT("InterchangeEditorPipelineModule", "InterchangeAssetCategoryCategoryName", "Interchange"));
	
	FCoreDelegates::OnPreExit.AddLambda([this]()
		{
			//We must release the resources before the application start to unload modules
			ReleaseResources();
		});
}

void FInterchangeEditorPipelinesModule::ReleaseResources()
{
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (FName ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}

		for (FName PropertyName : PropertiesTypesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
	}
	ClassesToUnregisterOnShutdown.Empty();
	PropertiesTypesToUnregisterOnShutdown.Empty();

	UnregisterPropertySectionMappings();

	InterchangeEditorPipelineStyle = nullptr;
}

TSharedRef<FPropertySection> FInterchangeEditorPipelinesModule::RegisterPropertySection(FPropertyEditorModule& PropertyModule, FName ClassName, FName SectionName, FText DisplayName)
{
	TSharedRef<FPropertySection> PropertySection = PropertyModule.FindOrCreateSection(ClassName, SectionName, DisplayName);
	RegisteredPropertySections.Add(ClassName, SectionName);

	return PropertySection;
}

void FInterchangeEditorPipelinesModule::RegisterPropertySectionMappings()
{
	const FName PropertyEditorModuleName("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);

	//We add number in front of the section name because the property editor is ordering alphabetically. Since we set the display name, the section name in the UI are correct.

	// Assets
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "InterchangeGenericAssetsPipeline", "General", LOCTEXT("General", "General"));
		Section->AddCategory("Common");
		Section->AddCategory("Conflicts");
	}

	// Meshes
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "InterchangeGenericAssetsPipeline", "0 Meshes", LOCTEXT("Meshes", "Meshes"));
		Section->AddCategory("Common Meshes");
	}

	// Static Meshes
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "InterchangeGenericMeshPipeline", "1 StaticMeshes", LOCTEXT("Static Meshes", "Static Meshes"));
		Section->AddCategory("Static Meshes");
	}

	// Skeletal Meshes
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "InterchangeGenericMeshPipeline", "2 SkeletalMeshes", LOCTEXT("Skeletal Meshes", "Skeletal Meshes"));
		Section->AddCategory("Common Skeletal Meshes and Animations");
		Section->AddCategory("Skeletal Meshes");
	}

	// Animation
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "InterchangeGenericAnimationPipeline", "3 AnimationSequences", LOCTEXT("Animations", "Animations"));
		Section->AddCategory("Common Skeletal Meshes and Animations");
		Section->AddCategory("Animations");
	}

	// Materials
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "InterchangeGenericMaterialPipeline", "4 Materials", LOCTEXT("Materials", "Materials"));
		Section->AddCategory("Materials");
	}

	// Textures
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "InterchangeGenericTexturePipeline", "5 Textures", LOCTEXT("Textures", "Textures"));
		Section->AddCategory("Textures");
		Section->AddCategory("Volumes");
	}
}

void FInterchangeEditorPipelinesModule::UnregisterPropertySectionMappings()
{
	const FName PropertyEditorModuleName("PropertyEditor");
	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(PropertyEditorModuleName);

	if (!PropertyModule)
	{
		return;
	}

	for (TMultiMap<FName, FName>::TIterator PropertySectionIterator = RegisteredPropertySections.CreateIterator(); PropertySectionIterator; ++PropertySectionIterator)
	{
		PropertyModule->RemoveSection(PropertySectionIterator->Key, PropertySectionIterator->Value);
		PropertySectionIterator.RemoveCurrent();
	}

	RegisteredPropertySections.Empty();
}

#undef LOCTEXT_NAMESPACE

