// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGLTFPipelineCustomizations.h"

#include "InterchangeglTFPipeline.h"
#include "Materials/MaterialInstance.h"

#include "DetailLayoutBuilder.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<IDetailCustomization> FInterchangeGLTFPipelineCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangeGLTFPipelineCustomization());
}

TSharedRef<IDetailCustomization> FInterchangeGLTFPipelineSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangeGLTFPipelineSettingsCustomization());
}

void FInterchangeGLTFPipelineCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	TWeakObjectPtr<UInterchangeGLTFPipeline> Pipeline = Cast<UInterchangeGLTFPipeline>(EditingObjects[0].Get());

	if (!ensure(Pipeline.IsValid()))
	{
		return;
	}

	IDetailCategoryBuilder& GLTFCategory = DetailBuilder.EditCategory("GLTF"); //glTF gets auto capitalized and separated into "Gl TF".

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	if (Pipeline->GLTFPipelineSettings)
	{
		Pipeline->GLTFPipelineSettings->SetMaterialParentsEditible(false);
	}

	if (Pipeline->CanEditPropertiesStates())
	{
		DetailsView->SetObject(Pipeline->GLTFPipelineSettings);

		GLTFCategory.AddCustomRow(NSLOCTEXT("InterchangeGLTFPipelineCustomization", "GLTFPredefinedMaterialLibrary", "Predefined Material Functions"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
			[
				DetailsView.ToSharedRef()
			]
			];
	}
	else
	{
		GLTFCategory.AddCustomRow(NSLOCTEXT("InterchangeGLTFPipelineMaterialSubstitution::Message::Row", "GLTFPredefinedMaterialSubstitutionMessageRow", "GLTF Pipeline Material Substitution Message"))
			.NameContent()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(NSLOCTEXT("InterchangeGLTFPipelineCustomization::Message", "GLTFPredefinedMaterialSubstitutionMessage", "MaterialInstance Parent Material Substitution can be\ncustomized in 'Project Settings > Interchange GLTF'."))
				.AutoWrapText(true)
			];
	}
	
}

void FInterchangeGLTFPipelineSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	GLTFPipelineSettings = Cast<UGLTFPipelineSettings>(EditingObjects[0].Get());

	if (!ensure(GLTFPipelineSettings.IsValid()))
	{
		return;
	}
	
	TSharedRef< IPropertyHandle > PairingsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UGLTFPipelineSettings, MaterialParents));
	if (!PairingsHandle->IsValidHandle())
	{
		return;
	}

	DetailBuilder.HideProperty(PairingsHandle);

	uint32 NumChildren = 0;
	PairingsHandle->GetNumChildren(NumChildren);

	for(uint32 i = 0; i < NumChildren; ++i)
	{
		TSharedPtr<IPropertyHandle> ChildPropertyHandle = PairingsHandle->GetChildHandle(i);
		TSharedPtr<IPropertyHandle> KeyPropertyHandle = ChildPropertyHandle->GetKeyHandle();

		IDetailPropertyRow& GLTFPredefinedMaterials = DetailBuilder.AddPropertyToCategory(ChildPropertyHandle.ToSharedRef());
		GLTFPredefinedMaterials.ShowPropertyButtons(false);
		GLTFPredefinedMaterials.IsEnabled(GLTFPipelineSettings->IsMaterialParentsEditible());

		FDetailWidgetRow& DetailWidgetRow = GLTFPredefinedMaterials.CustomWidget();
		DetailWidgetRow
		.NameContent()
		[
			SNullWidget::NullWidget
		]
		.ValueContent()
		.MinDesiredWidth(1.0f)
		.MaxDesiredWidth(1.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(UMaterialInstance::StaticClass())
				.PropertyHandle(ChildPropertyHandle)
			]
		];
	}

	//Reset Editibility for Project Settings -> Interchange GLTF
	GLTFPipelineSettings->SetMaterialParentsEditible(true);

	IDetailCategoryBuilder& GLTFPredefinedMaterialLibraryCategory = DetailBuilder.EditCategory("PredefinedglTFMaterialLibrary", FText::FromString("Predefined glTF Material Library"));

	DetailBuilder.HideCategory(GET_MEMBER_NAME_CHECKED(UGLTFPipelineSettings, MaterialParents));
}

