// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class UGLTFPipelineSettings;
struct FAssetData;

class FInterchangeGLTFPipelineSettingsCustomization : public IDetailCustomization
{
public:

	/**
	 * Creates an instance of this class
	 */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */

private:
	TWeakObjectPtr<UGLTFPipelineSettings> GLTFPipelineSettings;
};

class FInterchangeGLTFPipelineCustomization : public IDetailCustomization
{
public:

	/**
	 * Creates an instance of this class
	 */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */
};