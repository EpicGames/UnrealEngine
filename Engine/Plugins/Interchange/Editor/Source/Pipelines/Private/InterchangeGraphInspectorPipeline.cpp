// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeGraphInspectorPipeline.h"


#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/App.h"
#include "SInterchangeGraphInspectorWindow.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGraphInspectorPipeline)

void UInterchangeGraphInspectorPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath)
{
	//Do not pop dialog if we are unattended or doing automation
	if (FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript)
	{
		return;
	}

	//Create and show the graph inspector UI dialog
	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}
	TSharedRef<SWindow> Window = SNew(SWindow)
		.ClientSize(FVector2D(800.f, 650.f))
		.Title(NSLOCTEXT("Interchange", "GraphInspectorTitle", "Interchange node graph inspector"));
	TSharedPtr<SInterchangeGraphInspectorWindow> InterchangeGraphInspectorWindow;

	Window->SetContent
	(
		SAssignNew(InterchangeGraphInspectorWindow, SInterchangeGraphInspectorWindow)
		.InterchangeBaseNodeContainer(BaseNodeContainer)
		.OwnerWindow(Window)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
}

