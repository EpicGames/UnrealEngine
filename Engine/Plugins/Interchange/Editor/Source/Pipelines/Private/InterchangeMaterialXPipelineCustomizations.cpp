// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMaterialXPipelineCustomizations.h"

#include "InterchangeMaterialXPipeline.h"

#include "DetailLayoutBuilder.h"
#include "Engine/RendererSettings.h"
#include "Materials/MaterialFunction.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<IDetailCustomization> FInterchangeMaterialXPipelineCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangeMaterialXPipelineCustomization());
}

TSharedRef<IDetailCustomization> FInterchangeMaterialXPipelineSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangeMaterialXPipelineSettingsCustomization());
}

void FInterchangeMaterialXPipelineCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	TWeakObjectPtr<UInterchangeMaterialXPipeline> Pipeline = Cast<UInterchangeMaterialXPipeline>(EditingObjects[0].Get());

	if (!ensure(Pipeline.IsValid()))
	{
		return;
	}

	IDetailCategoryBuilder& MaterialXCategory = DetailBuilder.EditCategory("MaterialX");

	MaterialXCategory.SetDisplayName(NSLOCTEXT("InterchangeMaterialXPipelineCustomization", "CategoryDisplayName", "MaterialX Settings"));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	MaterialXCategory.AddCustomRow(NSLOCTEXT("InterchangeMaterialXPipelineMaterialSubstitution::Message::Row", "MaterialXPredefinedMaterialSubstitutionMessageRow", "MaterialX Pipeline Material Substitution Message"))
	.NameContent()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(NSLOCTEXT("InterchangeMaterialXPipelineCustomization::Message", "MaterialXPredefinedMaterialSubstitutionMessage", "See 'Project Settings > Engine > Interchange MaterialX' to edit settings."))
		.AutoWrapText(false)
	];
}

void FInterchangeMaterialXPipelineSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	MaterialXSettings = Cast<UMaterialXPipelineSettings>(EditingObjects[0].Get());

	if (!ensure(MaterialXSettings.IsValid()))
	{
		return;
	}

	auto Customize = [&DetailBuilder](const FName& MemberName, const FName& Category, const FText& DisplayName, uint8 EnumType)
	{
		TSharedRef< IPropertyHandle > PairingsHandle = DetailBuilder.GetProperty(MemberName);
		if(!PairingsHandle->IsValidHandle())
		{
			return;
		}

		DetailBuilder.HideProperty(PairingsHandle);

		IDetailCategoryBuilder& MaterialXPredefinedCategory = DetailBuilder.EditCategory(Category, DisplayName);		

		uint32 NumChildren = 0;
		PairingsHandle->GetNumChildren(NumChildren);

		for(uint32 i = 0; i < NumChildren; ++i)
		{
			TSharedPtr<IPropertyHandle> ChildPropertyHandle = PairingsHandle->GetChildHandle(i);
			TSharedPtr<IPropertyHandle> KeyPropertyHandle = ChildPropertyHandle->GetKeyHandle();
			
			UEnum* Enum = CastField<FEnumProperty>(KeyPropertyHandle->GetProperty())->GetEnum();

			uint8 EnumValue = 0;
			KeyPropertyHandle->GetValue(EnumValue);

			FText EnumString;
			KeyPropertyHandle->GetValueAsDisplayText(EnumString);

			FDetailWidgetRow& DetailWidgetRow = MaterialXPredefinedCategory.AddCustomRow(FText::FromStringView(TEXT("MaterialX")));
			DetailWidgetRow
			.NameContent()
			[
				SNew(SEditableText)
				.Text(EnumString)
				.IsReadOnly(true)
				.ToolTipText(Enum->GetToolTipTextByIndex(EnumValue))
			]
			.ValueContent()
			.MinDesiredWidth(1.0f)
			.MaxDesiredWidth(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UMaterialFunction::StaticClass())
					.PropertyHandle(ChildPropertyHandle)
					.OnShouldFilterAsset_Static(&OnShouldFilterAssetEnum, EnumType, EnumValue)
				]
			];
		}
	};

	Customize(GET_MEMBER_NAME_CHECKED(UMaterialXPipelineSettings, PredefinedSurfaceShaders),
			  TEXT("MaterialXPredefined | Surface Shaders"),
			  NSLOCTEXT("InterchangeMaterialXPipelineSettingsCustomization", "MaterialXPredefined | SurfaceShaders", "Surface Shaders"),
			  UE::Interchange::MaterialX::IndexSurfaceShaders);

	Customize(GET_MEMBER_NAME_CHECKED(UMaterialXPipelineSettings, PredefinedBSDF),
			  TEXT("MaterialXPredefined | BSDF"),
			  NSLOCTEXT("InterchangeMaterialXPipelineSettingsCustomization", "MaterialXPredefined | BSDF", "Bidirectional Scattering Distribution Functions"),
			  UE::Interchange::MaterialX::IndexBSDF);

	Customize(GET_MEMBER_NAME_CHECKED(UMaterialXPipelineSettings, PredefinedEDF),
			  TEXT("MaterialXPredefined | EDF"),
			  NSLOCTEXT("InterchangeMaterialXPipelineSettingsCustomization", "MaterialXPredefined | EDF", "Emission Distribution Functions"),
			  UE::Interchange::MaterialX::IndexEDF);

	Customize(GET_MEMBER_NAME_CHECKED(UMaterialXPipelineSettings, PredefinedVDF),
			  TEXT("MaterialXPredefined | VDF"),
			  NSLOCTEXT("InterchangeMaterialXPipelineSettingsCustomization", "MaterialXPredefined | VDF", "Volume Distribution Functions"),
			  UE::Interchange::MaterialX::IndexVDF);
}

bool FInterchangeMaterialXPipelineSettingsCustomization::OnShouldFilterAssetEnum(const FAssetData& InAssetData, uint8 EnumType, uint8 EnumValue)
{
	UMaterialXPipelineSettings::FMaterialXSettings::ValueType * Settings = UMaterialXPipelineSettings::SettingsInputsOutputs.Find(UMaterialXPipelineSettings::ToEnumKey(EnumType, EnumValue));

	if(Settings)
	{
		return UMaterialXPipelineSettings::ShouldFilterAssets(Cast<UMaterialFunction>(InAssetData.GetAsset()), Settings->Key, Settings->Value);
	}
	return false;
}
