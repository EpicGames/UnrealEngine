// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class UMaterialXPipelineSettings;
struct FAssetData;

class FInterchangeMaterialXPipelineSettingsCustomization : public IDetailCustomization
{
public:

	/**
	 * Creates an instance of this class
	 */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */

protected:
	static bool OnShouldFilterAssetEnum(const FAssetData&, uint8 EnumType, uint8 EnumValue);

private:
	TWeakObjectPtr<UMaterialXPipelineSettings> MaterialXSettings;
};

class FInterchangeMaterialXPipelineCustomization : public IDetailCustomization
{
public:

	/**
	 * Creates an instance of this class
	 */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */
};