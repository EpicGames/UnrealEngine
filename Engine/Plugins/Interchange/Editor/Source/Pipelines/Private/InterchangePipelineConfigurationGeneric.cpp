// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangePipelineConfigurationGeneric.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "SInterchangePipelineConfigurationDialog.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangePipelineConfigurationGeneric)


namespace UE::Interchange::Private
{
	float GetMinSizeX() { return 550.f; }
	float GetMinSizeY() { return 500.f; }

	const FVector2D& DefaultClientSize()
	{
		static FVector2D DefaultClientSize(1000.0, 650.0);
		return DefaultClientSize;
	}

	const FString& GetSizeXUid()
	{
		static FString SizeXStr = TEXT("SizeX");
		return SizeXStr;
	}

	const FString& GetSizeYUid()
	{
		static FString SizeYStr = TEXT("SizeY");
		return SizeYStr;
	}

	void RestoreClientSize(FVector2D& OutClientSize, const FString& WindowsUniqueID)
	{
		if (GConfig->DoesSectionExist(TEXT("InterchangeImportDialogOptions"), GEditorPerProjectIni))
		{
			FString SizeXUid = WindowsUniqueID + GetSizeXUid();
			FString SizeYUid = WindowsUniqueID + GetSizeYUid();
			GConfig->GetDouble(TEXT("InterchangeImportDialogOptions"), *SizeXUid, OutClientSize.X, GEditorPerProjectIni);
			GConfig->GetDouble(TEXT("InterchangeImportDialogOptions"), *SizeYUid, OutClientSize.Y, GEditorPerProjectIni);
		}
	}

	void BackupClientSize(SWindow* Window, const FString& WindowsUniqueID)
	{
		if (!Window)
		{
			return;
		}

		// Need to convert it back so that on creating the window, it will appropriately adjust for DPI scale.
		const FVector2D ClientSize = Window->GetClientSizeInScreen() / Window->GetDPIScaleFactor();

		FString SizeXUid = WindowsUniqueID + GetSizeXUid();
		FString SizeYUid = WindowsUniqueID + GetSizeYUid();
		GConfig->SetDouble(TEXT("InterchangeImportDialogOptions"), *SizeXUid, ClientSize.X, GEditorPerProjectIni);
		GConfig->SetDouble(TEXT("InterchangeImportDialogOptions"), *SizeYUid, ClientSize.Y, GEditorPerProjectIni);
	}

	FString GetWindowsUID(bool bSceneImport, bool bInvokedThroughTestPlan)
	{
		if (bInvokedThroughTestPlan)
		{
			return TEXT("TestPlanConfigurationDialog");
		}

		if (bSceneImport)
		{
			return TEXT("ImportSceneDialog");
		}

		return TEXT("ImportContentDialog");
	}
}

EInterchangePipelineConfigurationDialogResult UInterchangePipelineConfigurationGeneric::ShowPipelineDialog_Internal(FPipelineConfigurationDialogParams& InParams)
{
	using namespace UE::Interchange::Private;
	//Create and show the graph inspector UI dialog
	TSharedPtr<SWindow> ParentWindow;
	if (IMainFrameModule* MainFrame = FModuleManager::LoadModulePtr<IMainFrameModule>("MainFrame"))
	{
		ParentWindow = MainFrame->GetParentWindow();
	}

	const FString WindowDialogUid = GetWindowsUID(InParams.bSceneImport, InParams.bInvokedThroughTestPlan);
	FVector2D ClientSize = DefaultClientSize();
	RestoreClientSize(ClientSize, WindowDialogUid);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.ClientSize(ClientSize)
		.MinWidth(GetMinSizeX())
		.MinHeight(GetMinSizeY())
		.Title_Lambda([bSceneImport = InParams.bSceneImport, bReimport = InParams.bReimport, bInvokedThroughTestPlan = InParams.bInvokedThroughTestPlan]() 
		{
			if (bInvokedThroughTestPlan)
			{
				return NSLOCTEXT("Interchange", "PipelineConfigurationGenericTitlePipelineConfiguration", "Pipeline Configuration");
			}

			if (bReimport)
			{
				return NSLOCTEXT("Interchange", "PipelineConfigurationGenericTitleReimportContent", "Reimport Content");
			}

			if (bSceneImport)
			{
				return NSLOCTEXT("Interchange", "PipelineConfigurationGenericTitleScene", "Import Scene");
			}
			
			return NSLOCTEXT("Interchange", "PipelineConfigurationGenericTitleContent", "Import Content");
		});

	TSharedPtr<SInterchangePipelineConfigurationDialog> InterchangePipelineConfigurationDialog = SNew(SInterchangePipelineConfigurationDialog)
		.OwnerWindow(Window)
		.SourceData(InParams.SourceData)
		.bSceneImport(InParams.bSceneImport)
		.bReimport(InParams.bReimport)
		.bTestConfigDialog(InParams.bInvokedThroughTestPlan)
		.PipelineStacks(InParams.PipelineStacks)
		.OutPipelines(&InParams.OutPipelines)
		.BaseNodeContainer(InParams.BaseNodeContainer)
		.ReimportObject(InParams.ReimportAsset)
		.Translator(InParams.Translator)
		.bOverrideDefaultShowEssentials(InParams.bOverrideDefaultShowEssentials)
		.bOverrideDefaultFilterOnContent(InParams.bOverrideDefaultFilterOnContent);

	Window->SetContent(InterchangePipelineConfigurationDialog.ToSharedRef());

	Window->SetOnWindowClosed(FOnWindowClosed::CreateLambda([&WindowDialogUid](const TSharedRef<SWindow>& WindowMoved)
		{
			BackupClientSize(&WindowMoved.Get(), WindowDialogUid);
		}));

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	if (InterchangePipelineConfigurationDialog->IsCanceled())
	{
		return EInterchangePipelineConfigurationDialogResult::Cancel;
	}

	if (InParams.bInvokedThroughTestPlan)
	{
		return EInterchangePipelineConfigurationDialogResult::SaveConfig;
	}
	else
	{
		if (InterchangePipelineConfigurationDialog->IsImportAll())
		{
			return EInterchangePipelineConfigurationDialogResult::ImportAll;
		}

		return EInterchangePipelineConfigurationDialogResult::Import;
	}
}