// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangePipelineFactories.h"

#include "Engine/BlueprintGeneratedClass.h"
#include "InterchangeEditorBlueprintPipelineBase.h"
#include "InterchangeBlueprintPipelineBase.h"
#include "InterchangePythonPipelineBase.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Misc/MessageDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangePipelineFactories)

namespace UE::Interchange::Private
{
	FText GetInterchangeCategoryPath()
	{
		return NSLOCTEXT("InterchangeEditorPipeline", "GetInterchangeCategoryPath", "Interchange");
	}
}

/**
 * UInterchangeBlueprintPipelineBaseFactory implementation
 */

UInterchangeBlueprintPipelineBaseFactory::UInterchangeBlueprintPipelineBaseFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	SupportedClass = UInterchangeBlueprintPipelineBase::StaticClass();
	ParentClass = UInterchangePipelineBase::StaticClass();
}

UObject* UInterchangeBlueprintPipelineBaseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a gameplay ability blueprint, then create and init one
	check(Class->IsChildOf(UInterchangeBlueprintPipelineBase::StaticClass()));

	// If they selected an interface, force the parent class to be UInterface
	if (BlueprintType == BPTYPE_Interface)
	{
		ParentClass = UInterface::StaticClass();
	}

	if ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf(UInterchangePipelineBase::StaticClass()))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : NSLOCTEXT("UInterchangeBlueprintPipelineBaseFactory", "Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("UInterchangeBlueprintPipelineBaseFactory", "CannotCreateInterchangeBlueprintPipelineBase", "Cannot create an Interchange Blueprint Pipeline based on the class '{ClassName}'."), Args));
		return nullptr;
	}
	else
	{
		return CastChecked<UInterchangeBlueprintPipelineBase>(FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UInterchangeBlueprintPipelineBase::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext));
	}
}

UObject* UInterchangeBlueprintPipelineBaseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}

/**
 * UInterchangeEditorBlueprintPipelineBaseFactory implementation
 */

UInterchangeEditorBlueprintPipelineBaseFactory::UInterchangeEditorBlueprintPipelineBaseFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	SupportedClass = UInterchangeEditorBlueprintPipelineBase::StaticClass();
	ParentClass = UInterchangeEditorPipelineBase::StaticClass();
}

UObject* UInterchangeEditorBlueprintPipelineBaseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a gameplay ability blueprint, then create and init one
	check(Class->IsChildOf(UInterchangeEditorBlueprintPipelineBase::StaticClass()));

	// If they selected an interface, force the parent class to be UInterface
	if (BlueprintType == BPTYPE_Interface)
	{
		ParentClass = UInterface::StaticClass();
	}

	if ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf(UInterchangeEditorPipelineBase::StaticClass()))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : NSLOCTEXT("UInterchangeEditorBlueprintPipelineBaseFactory", "Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("UInterchangeEditorBlueprintPipelineBaseFactory", "CannotCreateInterchangeEditorBlueprintPipelineBase", "Cannot create an Interchange Editor Blueprint Pipeline based on the class '{ClassName}'."), Args));
		return nullptr;
	}
	else
	{
		return CastChecked<UInterchangeEditorBlueprintPipelineBase>(FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UInterchangeEditorBlueprintPipelineBase::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext));
	}
}

UObject* UInterchangeEditorBlueprintPipelineBaseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}

/**
 * UInterchangePipelineBaseFactory implementation
 */

UInterchangePipelineBaseFactory::UInterchangePipelineBaseFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	bEditorImport = false;
	SupportedClass = UInterchangePipelineBase::StaticClass();
}

UObject* UInterchangePipelineBaseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* Context, FFeedbackContext* Warn)
{
	UInterchangePipelineBase* Pipeline = nullptr;
	if (PipelineClass && PipelineClass->IsChildOf(SupportedClass))
	{
		Pipeline = NewObject<UInterchangePipelineBase>(InParent, PipelineClass, InName, InFlags | RF_Transactional);
	}
	return Pipeline;
}

FText UInterchangePipelineBaseFactory::GetDisplayName() const
{
	return NSLOCTEXT("UInterchangePipelineBaseFactory", "MenuEntry", "Interchange Pipeline");
}

bool UInterchangePipelineBaseFactory::ConfigureProperties()
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FInterchangePipelineBaseFilterViewer> Filter = MakeShared<FInterchangePipelineBaseFilterViewer>();
	Options.ClassFilters.Add(Filter.ToSharedRef());

	//CLASS_HideDropDown prevent sub pipeline to show in the list
	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_HideDropDown | CLASS_Hidden | CLASS_Transient;
	Filter->AllowedChildrenOfClasses.Add(UInterchangePipelineBase::StaticClass());
	//Blueprint pipeline have there own factory
	Filter->DisallowedChildrenOfClasses.Add(UInterchangeBlueprintPipelineBase::StaticClass());
	Filter->DisallowedChildrenOfClasses.Add(UInterchangeEditorBlueprintPipelineBase::StaticClass());

	const FText TitleText = NSLOCTEXT("UInterchangePipelineBaseFactory", "CreateOptions", "Pick a Pipeline Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UInterchangePipelineBase::StaticClass());

	if (bPressedOk)
	{
		PipelineClass = ChosenClass;
	}

	return bPressedOk;
}

/**
 * UInterchangePythonPipelineFactory implementation
 */

UInterchangePythonPipelineAssetFactory::UInterchangePythonPipelineAssetFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	bEditorImport = false;
	SupportedClass = UInterchangePythonPipelineAsset::StaticClass();
}

UObject* UInterchangePythonPipelineAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* Context, FFeedbackContext* Warn)
{
	UInterchangePythonPipelineAsset* Pipeline = nullptr;
	if (PythonClass && PythonClass->IsChildOf(UInterchangePipelineBase::StaticClass()))
	{
		Pipeline = NewObject<UInterchangePythonPipelineAsset>(InParent, SupportedClass, InName, InFlags | RF_Transactional);
		//Python pipeline are editor only package
		Pipeline->GetPackage()->SetPackageFlags(PKG_EditorOnly);
		Pipeline->PythonClass = PythonClass;
		Pipeline->GeneratePipeline();
	}
	return Pipeline;
}

FText UInterchangePythonPipelineAssetFactory::GetDisplayName() const
{
	return NSLOCTEXT("UInterchangePythonPipelineFactory", "MenuEntry", "Interchange Python Pipeline");
}

bool UInterchangePythonPipelineAssetFactory::ConfigureProperties()
{

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FInterchangePipelineBaseFilterViewer> Filter = MakeShared<FInterchangePipelineBaseFilterViewer>();
	Options.ClassFilters.Add(Filter.ToSharedRef());

	//CLASS_HideDropDown prevent sub pipeline to show in the list
	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_HideDropDown | CLASS_Hidden | CLASS_Transient;

	Filter->AllowedChildrenOfClasses.Add(UInterchangePythonPipelineBase::StaticClass());

	//Blueprint pipeline have there own factory
	Filter->DisallowedChildrenOfClasses.Add(UInterchangeBlueprintPipelineBase::StaticClass());
	Filter->DisallowedChildrenOfClasses.Add(UInterchangeEditorBlueprintPipelineBase::StaticClass());

	const FText TitleText = NSLOCTEXT("UInterchangePythonPipelineAssetFactory", "CreateOptions", "Pick a Pipeline Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UInterchangePipelineBase::StaticClass());

	if (bPressedOk)
	{
		PythonClass = ChosenClass;
	}

	return bPressedOk;
}
