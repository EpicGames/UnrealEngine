// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetTypeActions_Blueprint.h"
#include "ClassViewerFilter.h"
#include "Factories/Factory.h"

#include "InterchangePipelineFactories.generated.h"

class FInterchangePipelineBaseFilterViewer : public IClassViewerFilter
{
public:
	TSet<const UClass*> AllowedChildrenOfClasses;
	TSet<const UClass*> DisallowedChildrenOfClasses;
	EClassFlags DisallowedClassFlags = EClassFlags::CLASS_None;
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<class FClassViewerFilterFuncs> InFilterFuncs) override
	{
		if (InClass->HasAnyClassFlags(DisallowedClassFlags))
		{
			return false;
		}
		
		return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed
			&& InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InClass) == EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		if (InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags))
		{
			return false;
		}

		return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed
			&& InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InUnloadedClassData) == EFilterReturn::Failed;
	}
};

UCLASS(hidecategories = Object, collapsecategories)
class UInterchangeBlueprintPipelineBaseFactory : public UFactory
{
	GENERATED_BODY()

public:
	UInterchangeBlueprintPipelineBaseFactory();

	// The type of blueprint that will be created
	UPROPERTY(EditAnywhere, Category = InterchangeBlueprintPipelineBaseFactory)
	TEnumAsByte<enum EBlueprintType> BlueprintType;

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category = InterchangeBlueprintPipelineBaseFactory)
	TSubclassOf<class UInterchangePipelineBase> ParentClass;

	//Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//End UFactory Interface	
};

UCLASS(hidecategories = Object, collapsecategories)
class UInterchangeEditorBlueprintPipelineBaseFactory : public UFactory
{
	GENERATED_BODY()

public:
	UInterchangeEditorBlueprintPipelineBaseFactory();

	// The type of blueprint that will be created
	UPROPERTY(EditAnywhere, Category = InterchangeEditorBlueprintPipelineBaseFactory)
	TEnumAsByte<enum EBlueprintType> BlueprintType;

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category = InterchangeEditorBlueprintPipelineBaseFactory)
	TSubclassOf<class UInterchangeEditorPipelineBase> ParentClass;

	//Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//End UFactory Interface	
};

UCLASS(hidecategories = Object, collapsecategories)
class UInterchangePipelineBaseFactory : public UFactory
{
	GENERATED_BODY()

public:
	UInterchangePipelineBaseFactory();

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	// End of UFactory Interface
private:
	UClass* PipelineClass = nullptr;
};

UCLASS(hidecategories = Object, collapsecategories)
class UInterchangePythonPipelineAssetFactory : public UFactory
{
	GENERATED_BODY()

public:
	UInterchangePythonPipelineAssetFactory();

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	// End of UFactory Interface
private:
	UClass* PythonClass = nullptr;
};
