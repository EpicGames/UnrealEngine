// Copyright Epic Games, Inc. All Rights Reserved.
#include "SInterchangeAssetCard.h"

#include "GameFramework/Actor.h"
#include "InterchangeEditorPipelineStyle.h"
#include "InterchangePipelineBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "SSimpleButton.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "InterchangeAssetCard"


/************************************************************************/
/* SInterchangeAssetCard Implementation                    */
/************************************************************************/

SInterchangeAssetCard::~SInterchangeAssetCard()
{

}

void SInterchangeAssetCard::RefreshCard(UInterchangeBaseNodeContainer* InPreviewNodeContainer)
{
	if (bCardDisabled)
	{
		return;
	}

	check(InPreviewNodeContainer);
	CardAssetCount = 0;
	CardAssetToImportCount = 0;
	CardAssetDisabledCount = 0;
	CardTooltip.Reset();
	FString ImportAssets;
	FString IgnoreAssets;

	auto FillStringData = [](FString& StringData, const FString& DisplayLabel)
		{
			StringData += TEXT("\n\t");
			StringData += DisplayLabel;
		};

	//Query the data we need to update the card
	InPreviewNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([this, &FillStringData, &ImportAssets, &IgnoreAssets](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
		{
			if (!FactoryNode || !FactoryNode->GetObjectClass() || !FactoryNode->GetObjectClass()->IsChildOf(AssetClass))
			{
				return;
			}

			CardAssetCount++;
			if (FactoryNode->IsEnabled())
			{
				FillStringData(ImportAssets, FactoryNode->GetDisplayLabel());
				CardAssetToImportCount++;
			}
			else
			{
				FillStringData(IgnoreAssets, FactoryNode->GetDisplayLabel());
				CardAssetDisabledCount++;
			}
		});

	if (!ImportAssets.IsEmpty())
	{
		CardTooltip += LOCTEXT("CardImportAssetTooltipPrefix", "Import Assets:").ToString();
		CardTooltip += ImportAssets;
	}
	if (!IgnoreAssets.IsEmpty())
	{
		CardTooltip += LOCTEXT("CardIgnoreAssetTooltipPrefix", "Ignore Assets:").ToString();
		CardTooltip += IgnoreAssets;
	}
}

bool SInterchangeAssetCard::RefreshHasConflicts(const TArray<FInterchangeConflictInfo>& InConflictInfos)
{
	if (bCardDisabled)
	{
		return false;
	}

	for (const FInterchangeConflictInfo& ConflictInfo : InConflictInfos)
	{
		if (ConflictInfo.AffectedAssetClasses.Contains(AssetClass))
		{
			bHasConflictWarnings = true;
			return true;
		}
	}

	bHasConflictWarnings = false;
	return false;
}

void SInterchangeAssetCard::Construct(const FArguments& InArgs)
{
	//Make sure we have a valid node container
	check(InArgs._PreviewNodeContainer);
	AssetClass = InArgs._AssetClass;
	check(AssetClass != nullptr);
	ShouldImportAssetType = InArgs._ShouldImportAssetType;
	check(ShouldImportAssetType.IsBound());
	OnImportAssetTypeChanged = InArgs._OnImportAssetTypeChanged;
	check(OnImportAssetTypeChanged.IsBound());
	bCardDisabled = InArgs._CardDisabled;

	RefreshCard(InArgs._PreviewNodeContainer);

	const FSlateBrush* CardAssetIcon = nullptr;
	CardAssetIcon = FSlateIconFinder::FindIconBrushForClass(AssetClass);

	TSharedPtr<SImage> CardIconWidget = SNew(SImage)
		.Image(CardAssetIcon)
		.Visibility(CardAssetIcon != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed);


	FText CardImportText = FText::Format(LOCTEXT("CardImportText", "Import {0}"), FText::FromString(AssetClass->GetName()));

	FName IconWarningName = "Icons.Alert.Solid";
	const FSlateIcon IconWarning = FSlateIconFinder::FindIcon(IconWarningName);

	const ISlateStyle* InterchangeEditorPipelineStyle = FSlateStyleRegistry::FindSlateStyle("InterchangeEditorPipelineStyle");
	
	const FSlateBrush* HeaderBorderBrush = nullptr;
	const FSlateBrush* BodyBorderBrush = nullptr;
	const FSlateBrush* BackgroundBorderBrush = nullptr;
	if (InterchangeEditorPipelineStyle)
	{
		HeaderBorderBrush = InterchangeEditorPipelineStyle->GetBrush("AssetCard.Header.Border");
		BodyBorderBrush = InterchangeEditorPipelineStyle->GetBrush("AssetCard.Body.Border");
		BackgroundBorderBrush = InterchangeEditorPipelineStyle->GetBrush("AssetCardList.Background.Border");
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SBorder)
			.BorderImage(HeaderBorderBrush)
			.IsEnabled(!bCardDisabled && (CardAssetToImportCount > 0))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.AutoWidth()
				.Padding(8.0f, 4.0f, 4.0f, 4.0f)
				[
					CardIconWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				.Padding(4.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("NormalFont"))
					.Text_Lambda([this]()
						{
							if (CardAssetToImportCount == 0)
							{
								return FText::Format(LOCTEXT("CardLabelText_NoAsset", "{0} (Asset type not found in source or disabled in this pipeline stack)"), FText::FromString(AssetClass->GetName()));
							}
							else
							{
								return FText::Format(LOCTEXT("CardLabelText_ValidAssets", "{0} ({1} {2})"), FText::FromString(AssetClass->GetName()), FText::FromString(FString::FromInt(CardAssetToImportCount)), CardAssetToImportCount > 1 ? LOCTEXT("CardAssetsPlural", "assets") : LOCTEXT("CardAssetSingle", "asset"));
							}
						})
					.ToolTipText_Lambda([this]()
						{
							if (CardAssetToImportCount == 0)
							{
								return LOCTEXT("CardTooltip_NoAssets", "This might be because the source file doesn't have asset of this type or the import options are not set to import this type.");
							}
							else
							{
								return FText::FromString(CardTooltip);
							}
						})
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SSpacer)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]() {return bHasConflictWarnings ? EVisibility::Visible : EVisibility::Collapsed; })
					.ToolTipText(LOCTEXT("ConflictWarningTooltipText", "There are some conflicts generated while importing the source file. Go to Conflicts Section in Advanced Settings for more details."))	
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(IconWarning.GetOptionalIcon())
						.ColorAndOpacity(FStyleColors::Warning)
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.AutoWidth()
					.Padding(4.0f, 4.0f, 8.0f, 4.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ConflictWarningText", "Conflict Warnings"))
						.ColorAndOpacity(FStyleColors::Warning)
					]
					
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0.0f, 0.0f, 0.0f, 4.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(BodyBorderBrush)
			.IsEnabled(!bCardDisabled && (CardAssetToImportCount > 0))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(0.6f)
				.HAlign(HAlign_Left)
				.Padding(8.0f, 4.0f)
				[
					SNew(SBox)
					.Padding(0.0f)
					[
						SNew(STextBlock)
						.Text(CardImportText)
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f)
				.AutoWidth()
				[
					SNew(SSeparator)
					.Orientation(EOrientation::Orient_Vertical)
					.Thickness(1.0f)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.4f)
				.HAlign(HAlign_Left)
				.Padding(8.0f, 4.0f)
				[
					SNew(SBox)
					.Padding(0.)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]()
						{
							if (CardAssetToImportCount != 0 && ShouldImportAssetType.Execute())
							{
								return ECheckBoxState::Checked;
							}
							return ECheckBoxState::Unchecked;
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckBoxState)
						{
							OnImportAssetTypeChanged.Execute(CheckBoxState == ECheckBoxState::Checked);
						})
					]
				]
			]
		]
	];

}


void SInterchangeAssetCardList::Construct(const FArguments& InArgs)
{
	AssetCards = InArgs._AssetCards;
	check(AssetCards);

	AssetCardList = SNew(SListView<TSharedPtr<SInterchangeAssetCard>>)
		.SelectionMode(ESelectionMode::None)
		.ListItemsSource(AssetCards)
		.OnGenerateRow(this, &SInterchangeAssetCardList::MakeAssetCardListRowWidget);

	const ISlateStyle* InterchangeEditorPipelineStyle = FSlateStyleRegistry::FindSlateStyle("InterchangeEditorPipelineStyle");

	const FSlateBrush* BackgroundBorderBrush = nullptr;
	if (InterchangeEditorPipelineStyle)
	{
		BackgroundBorderBrush = InterchangeEditorPipelineStyle->GetBrush("AssetCardList.Background.Border");
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(8.0f, 4.0f)
		.BorderImage(BackgroundBorderBrush)
		[
			AssetCardList.ToSharedRef()
		]
	];
}

void SInterchangeAssetCardList::RefreshList(UInterchangeBaseNodeContainer* InPreviewNodeContainer)
{
	for (TSharedPtr<SInterchangeAssetCard> AssetCard : *AssetCards)
	{
		AssetCard->RefreshCard(InPreviewNodeContainer);
	}
	AssetCardList->RequestListRefresh();
}

TSharedRef<ITableRow> SInterchangeAssetCardList::MakeAssetCardListRowWidget(TSharedPtr<SInterchangeAssetCard> InElement, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<SInterchangeAssetCard>>, OwnerTable)
		[
			InElement.ToSharedRef()
		];
}
#undef LOCTEXT_NAMESPACE
