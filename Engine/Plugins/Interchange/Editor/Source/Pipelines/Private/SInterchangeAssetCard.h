// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "UObject/Class.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

class UInterchangeBaseNodeContainer;
struct FInterchangeConflictInfo;

DECLARE_DELEGATE_OneParam(FInterchangeCard_OnImportAssetTypeChanged, bool)
DECLARE_DELEGATE_RetVal(bool, FInterchangeCard_ShouldImportAssetType)

class SInterchangeAssetCard : public SCompoundWidget
{
public:
	~SInterchangeAssetCard();

	SLATE_BEGIN_ARGS(SInterchangeAssetCard)
		: _PreviewNodeContainer(nullptr)
		, _AssetClass(nullptr)
		, _CardDisabled(false)
		, _ShouldImportAssetType()
		, _OnImportAssetTypeChanged()
		{}
		SLATE_ARGUMENT(UInterchangeBaseNodeContainer*, PreviewNodeContainer)
		SLATE_ARGUMENT(UClass*, AssetClass)
		SLATE_ARGUMENT(bool, CardDisabled)
		SLATE_EVENT(FInterchangeCard_ShouldImportAssetType, ShouldImportAssetType)
		SLATE_EVENT(FInterchangeCard_OnImportAssetTypeChanged, OnImportAssetTypeChanged)
	SLATE_END_ARGS()

public:
	/** Construct this widget */
	void Construct(const FArguments& InArgs);

	void RefreshCard(UInterchangeBaseNodeContainer* InPreviewNodeContainer);

	bool RefreshHasConflicts(const TArray<FInterchangeConflictInfo>& InConflictInfos);

protected:

	int32 CardAssetCount = 0;
	int32 CardAssetToImportCount = 0;
	int32 CardAssetDisabledCount = 0;
	FString CardTooltip;
	bool bHasConflictWarnings = false;
	bool bCardDisabled = false;

	/** The factory asset class so we can know which kind of Unreal asset this card is for. */
	UClass* AssetClass = nullptr;

	/** Delegate to invoke when we update the should import asset type UI. */
	FInterchangeCard_ShouldImportAssetType ShouldImportAssetType;

	/** Delegate to invoke when should import asset type value change. */
	FInterchangeCard_OnImportAssetTypeChanged OnImportAssetTypeChanged;
};

class SInterchangeAssetCardList : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SInterchangeAssetCardList)
		: _AssetCards()
		{}
		SLATE_ARGUMENT(TArray<TSharedPtr<SInterchangeAssetCard>>*, AssetCards)
	SLATE_END_ARGS()

public:
	/** Construct this widget */
	void Construct(const FArguments& InArgs);

	void RefreshList(UInterchangeBaseNodeContainer* InPreviewNodeContainer);

private:

	TArray<TSharedPtr<SInterchangeAssetCard>>* AssetCards;

	TSharedRef<ITableRow> MakeAssetCardListRowWidget(TSharedPtr<SInterchangeAssetCard> InElement, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr <SListView<TSharedPtr<SInterchangeAssetCard>>> AssetCardList;
};
