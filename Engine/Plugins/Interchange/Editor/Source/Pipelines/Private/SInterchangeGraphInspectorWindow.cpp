// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInterchangeGraphInspectorWindow.h"

#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "DetailsViewArgs.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "GameFramework/Actor.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "InterchangeGraphInspector"

/************************************************************************/
/* SInterchangeGraphInspectorTreeView Implementation                    */
/************************************************************************/

SInterchangeGraphInspectorTreeView::~SInterchangeGraphInspectorTreeView()
{

}

void SInterchangeGraphInspectorTreeView::Construct(const FArguments& InArgs)
{
	InterchangeBaseNodeContainer = InArgs._InterchangeBaseNodeContainer;
	bPreview = InArgs._bPreview;
	OnSelectionChangedDelegate = InArgs._OnSelectionChangedDelegate;
	//Build the FbxNodeInfoPtr tree data
	check(InterchangeBaseNodeContainer != nullptr);

	TArray<FString> Roots;
	InterchangeBaseNodeContainer->GetRoots(Roots);
	for (FString RootID : Roots)
	{
		const UInterchangeBaseNode* RootNode = InterchangeBaseNodeContainer->GetNode(RootID);
		if (bPreview)
		{
			const UInterchangeFactoryBaseNode* FactoryNode = Cast<UInterchangeFactoryBaseNode>(RootNode);
			if (!FactoryNode || !FactoryNode->GetObjectClass())
			{
				continue;
			}
		}
		RootNodeArray.Add(const_cast<UInterchangeBaseNode*>(RootNode));
	}

	STreeView::Construct
	(
		STreeView::FArguments()
		.TreeItemsSource(&RootNodeArray)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SInterchangeGraphInspectorTreeView::OnGenerateRowGraphInspectorTreeView)
		.OnGetChildren(this, &SInterchangeGraphInspectorTreeView::OnGetChildrenGraphInspectorTreeView)
		.OnContextMenuOpening(this, &SInterchangeGraphInspectorTreeView::OnOpenContextMenu)
		.OnSelectionChanged(this, &SInterchangeGraphInspectorTreeView::OnTreeViewSelectionChanged)
	);
}

/** The item used for visualizing the class in the tree. */
class SInterchangeGraphInspectorTreeViewItem : public STableRow< UInterchangeBaseNode* >
{
public:

	SLATE_BEGIN_ARGS(SInterchangeGraphInspectorTreeViewItem)
		: _InterchangeNode(nullptr)
		, _InterchangeBaseNodeContainer(nullptr)
		, _bPreview(false)
	{}

	/** The item content. */
	SLATE_ARGUMENT(UInterchangeBaseNode*, InterchangeNode)
	SLATE_ARGUMENT(UInterchangeBaseNodeContainer*, InterchangeBaseNodeContainer)
	SLATE_ARGUMENT(bool, bPreview)
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		InterchangeNode = InArgs._InterchangeNode;
		bPreview = InArgs._bPreview;
		InterchangeBaseNodeContainer = InArgs._InterchangeBaseNodeContainer;

		// This is supposed to always be valid
		check(InterchangeNode);
		check(InterchangeBaseNodeContainer);

		FString Tooltip = InterchangeNode->GetDisplayLabel();
		const FSlateBrush* TypeIcon = nullptr;
		FName IconName = InterchangeNode->GetIconName();
		if (IconName != NAME_None)
		{
			const FSlateIcon SlateIcon = FSlateIconFinder::FindIcon(IconName);
			TypeIcon = SlateIcon.GetOptionalIcon();
		}

		// Factory nodes take their icons directly from the class they represent, regardless of any icon which may be defined
		// @TODO: allow GetIconName() to override this for factory nodes?
		if (UInterchangeFactoryBaseNode* FactoryNode = Cast<UInterchangeFactoryBaseNode>(InterchangeNode))
		{
			if (UClass* IconClass = FactoryNode->GetObjectClass())
			{
				TypeIcon = FSlateIconFinder::FindIconBrushForClass(IconClass);
				Tooltip += TEXT(" [") + IconClass->GetName() + TEXT("]");
			}
		}

		if (!TypeIcon)
		{
			TypeIcon = FSlateIconFinder::FindIconBrushForClass(AActor::StaticClass());
		}

		TSharedPtr<SImage> IconWidget = nullptr;

		if (!bPreview)
		{
			FSlateColor TypeIconColor;
			switch (InterchangeNode->GetNodeContainerType())
			{
			case EInterchangeNodeContainerType::TranslatedAsset:
				TypeIconColor = FStyleColors::AccentBlue;
				break;
			case EInterchangeNodeContainerType::TranslatedScene:
				TypeIconColor = FStyleColors::AccentGreen;
				break;
			case EInterchangeNodeContainerType::FactoryData:
				TypeIconColor = FStyleColors::AccentPurple;
				break;
			case EInterchangeNodeContainerType::None:
			default:
				TypeIconColor = FStyleColors::AccentRed;
				break;
			}
			
			IconWidget = SNew(SImage)
				.Image(TypeIcon)
				.ColorAndOpacity(TypeIconColor)
				.Visibility(TypeIcon != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed);
		}
		else
		{
			IconWidget = SNew(SImage)
				.Image(TypeIcon)
				.Visibility(TypeIcon != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed);
		}

		this->ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SInterchangeGraphInspectorTreeViewItem::OnItemCheckChanged)
				.IsChecked(this, &SInterchangeGraphInspectorTreeViewItem::IsItemChecked)
				.IsEnabled(!bPreview)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 2.0f, 6.0f, 2.0f)
			[
				IconWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InterchangeNode->GetDisplayLabel()))
				.ToolTipText(FText::FromString(Tooltip))
			]

		];

		STableRow< UInterchangeBaseNode* >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true),
			InOwnerTableView
		);
	}

private:

	void RecursiveSetImport(UInterchangeBaseNode* Node, bool bState)
	{
		TArray<FString> Childrens = InterchangeBaseNodeContainer->GetNodeChildrenUids(Node->GetUniqueID());
		for (const FString& ChildID : Childrens)
		{
			UInterchangeBaseNode* ChildNode = const_cast<UInterchangeBaseNode*>(InterchangeBaseNodeContainer->GetNode(ChildID));
			if (!ChildNode)
				continue;
			ChildNode->SetEnabled(bState);
			RecursiveSetImport(ChildNode, bState);
		}
	}

	void OnItemCheckChanged(ECheckBoxState CheckType)
	{
		if (!InterchangeNode)
			return;
		const bool ImportState = CheckType == ECheckBoxState::Checked;
		InterchangeNode->SetEnabled(ImportState);
		RecursiveSetImport(InterchangeNode, ImportState);
	}

	ECheckBoxState IsItemChecked() const
	{
		return InterchangeNode->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/** The node to build the tree view row from. */
	UInterchangeBaseNode* InterchangeNode = nullptr;
	UInterchangeBaseNodeContainer* InterchangeBaseNodeContainer = nullptr;
	bool bPreview = false;
};

TSharedRef< ITableRow > SInterchangeGraphInspectorTreeView::OnGenerateRowGraphInspectorTreeView(UInterchangeBaseNode* Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SInterchangeGraphInspectorTreeViewItem > ReturnRow = SNew(SInterchangeGraphInspectorTreeViewItem, OwnerTable)
		.InterchangeNode(Item)
		.bPreview(bPreview)
		.InterchangeBaseNodeContainer(InterchangeBaseNodeContainer);
	return ReturnRow;
}
void SInterchangeGraphInspectorTreeView::OnGetChildrenGraphInspectorTreeView(UInterchangeBaseNode* InParent, TArray< UInterchangeBaseNode* >& OutChildren)
{
	TArray<FString> Childrens = InterchangeBaseNodeContainer->GetNodeChildrenUids(InParent->GetUniqueID());
	for (const FString& ChildID : Childrens)
	{
		const UInterchangeBaseNode* ChildNode = InterchangeBaseNodeContainer->GetNode(ChildID);
		if (!ChildNode || (bPreview && !ChildNode->IsA<UInterchangeFactoryBaseNode>()))
			continue;
		OutChildren.Add(const_cast<UInterchangeBaseNode*>(ChildNode));
	}
}

void SInterchangeGraphInspectorTreeView::RecursiveSetImport(UInterchangeBaseNode* Node, bool bState)
{
	if (!Node)
	{
		return;
	}

	Node->SetEnabled(bState);

	TArray<FString> Childrens = InterchangeBaseNodeContainer->GetNodeChildrenUids(Node->GetUniqueID());
	for (const FString& ChildID : Childrens)
	{
		UInterchangeBaseNode* ChildNode = const_cast<UInterchangeBaseNode*>(InterchangeBaseNodeContainer->GetNode(ChildID));
		if (!ChildNode)
			continue;
		RecursiveSetImport(ChildNode, bState);
	}
}

void SInterchangeGraphInspectorTreeView::OnToggleSelectAll(ECheckBoxState CheckType)
{
	for (UInterchangeBaseNode* Node : RootNodeArray)
	{
		if (!ensure(Node))
		{
			continue;
		}
		RecursiveSetImport(Node, CheckType == ECheckBoxState::Checked);
	}
}

void SInterchangeGraphInspectorTreeView::RecursiveSetExpand(UInterchangeBaseNode* Node, bool ExpandState)
{
	SetItemExpansion(Node, ExpandState);
	TArray<FString> Childrens = InterchangeBaseNodeContainer->GetNodeChildrenUids(Node->GetUniqueID());
	for (const FString& ChildID : Childrens)
	{
		UInterchangeBaseNode* ChildNode = const_cast<UInterchangeBaseNode*>(InterchangeBaseNodeContainer->GetNode(ChildID));
		if (!ChildNode)
			continue;
		RecursiveSetExpand(ChildNode, ExpandState);
	}
}

FReply SInterchangeGraphInspectorTreeView::OnExpandAll()
{
	for (UInterchangeBaseNode* Node : RootNodeArray)
	{
		if (!ensure(Node))
		{
			continue;
		}
		RecursiveSetExpand(Node, true);
	}
	return FReply::Handled();
}

FReply SInterchangeGraphInspectorTreeView::OnCollapseAll()
{
	for (UInterchangeBaseNode* Node : RootNodeArray)
	{
		if (!ensure(Node))
		{
			continue;
		}
		RecursiveSetExpand(Node, false);
	}
	return FReply::Handled();
}

TSharedPtr<SWidget> SInterchangeGraphInspectorTreeView::OnOpenContextMenu()
{
	if (bPreview)
	{
		return SNullWidget::NullWidget;
	}
	// Build up the menu for a selection
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());

	TArray<UInterchangeBaseNode*> SelectedNodes;
	const auto NumSelectedItems = GetSelectedItems(SelectedNodes);

	// We always create a section here, even if there is no parent so that clients can still extend the menu
	MenuBuilder.BeginSection("FbxSceneTreeViewContextMenuImportSection");
	{
		const FSlateIcon PlusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus");
		MenuBuilder.AddMenuEntry(LOCTEXT("CheckForImport", "Add Selection To Import"), FText(), PlusIcon, FUIAction(FExecuteAction::CreateSP(this, &SInterchangeGraphInspectorTreeView::AddSelectionToImport)));
		const FSlateIcon MinusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus");
		MenuBuilder.AddMenuEntry(LOCTEXT("UncheckForImport", "Remove Selection From Import"), FText(), MinusIcon, FUIAction(FExecuteAction::CreateSP(this, &SInterchangeGraphInspectorTreeView::RemoveSelectionFromImport)));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SInterchangeGraphInspectorTreeView::AddSelectionToImport()
{
	SetSelectionImportState(true);
}

void SInterchangeGraphInspectorTreeView::RemoveSelectionFromImport()
{
	SetSelectionImportState(false);
}

void SInterchangeGraphInspectorTreeView::SetSelectionImportState(bool MarkForImport)
{
	TArray<UInterchangeBaseNode*> SelectedNodes;
	GetSelectedItems(SelectedNodes);
	for (UInterchangeBaseNode* Item : SelectedNodes)
	{
		Item->SetEnabled(MarkForImport);
	}
}

void SInterchangeGraphInspectorTreeView::OnTreeViewSelectionChanged(UInterchangeBaseNode* Item, ESelectInfo::Type SelectionType)
{
	if (GetSelectionMode() == ESelectionMode::None)
	{
		return;
	}

	if (OnSelectionChangedDelegate.IsBound())
	{
		OnSelectionChangedDelegate.ExecuteIfBound(Item, SelectionType);
	}
}


/************************************************************************/
/* SInterchangeGraphInspectorWindow Implementation                      */
/************************************************************************/

SInterchangeGraphInspectorWindow::SInterchangeGraphInspectorWindow()
{
	InterchangeBaseNodeContainer = nullptr;
	GraphInspectorTreeview = nullptr;
	GraphInspectorDetailsView = nullptr;
	OwnerWindow = nullptr;
}

SInterchangeGraphInspectorWindow::~SInterchangeGraphInspectorWindow()
{
	InterchangeBaseNodeContainer = nullptr;
	GraphInspectorTreeview = nullptr;
	GraphInspectorDetailsView = nullptr;
	OwnerWindow = nullptr;
}

TSharedRef<SBox> SInterchangeGraphInspectorWindow::SpawnGraphInspector()
{
	//Create the treeview
	GraphInspectorTreeview = SNew(SInterchangeGraphInspectorTreeView)
		.InterchangeBaseNodeContainer(InterchangeBaseNodeContainer)
		.bPreview(bPreview)
		.OnSelectionChangedDelegate(this, &SInterchangeGraphInspectorWindow::OnSelectionChanged);

	TSharedPtr<SBox> InspectorBox;
	TSharedRef<SBox> GraphInspectorPanelBox = SNew(SBox)
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
		+ SSplitter::Slot()
		.Value(0.4f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SCheckBox)
						.HAlign(HAlign_Center)
						.OnCheckStateChanged(GraphInspectorTreeview.Get(), &SInterchangeGraphInspectorTreeView::OnToggleSelectAll)
						.Visibility(bPreview ? EVisibility::Collapsed : EVisibility::Visible)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("GraphInspectorWindow_Scene_All", "All"))
						.Visibility(bPreview ? EVisibility::Collapsed : EVisibility::Visible)
					]
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("FbxOptionWindow_Scene_ExpandAll", "Expand All"))
					.OnClicked(GraphInspectorTreeview.Get(), &SInterchangeGraphInspectorTreeView::OnExpandAll)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("FbxOptionWindow_Scene_CollapseAll", "Collapse All"))
					.OnClicked(GraphInspectorTreeview.Get(), &SInterchangeGraphInspectorTreeView::OnCollapseAll)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBox)
				[
					GraphInspectorTreeview.ToSharedRef()
				]
			]
		]
		+ SSplitter::Slot()
		.Value(0.6f)
		[
			SAssignNew(InspectorBox, SBox)
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	// TODO: @vjavdekar - hidden as this doesn't currently provide any value and existing open modal windows block the property matrix window from opening.
	// Reconsider this when the Preview Panel gets a UI Pass.
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bAllowSearch = bPreview;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	GraphInspectorDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(GraphInspectorDetailsView->AsShared());
	GraphInspectorDetailsView->SetObject(nullptr);
//	GraphInspectorDetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateLambda([this](const FPropertyAndParent&) { return bPreview; }));
//	GraphInspectorDetailsView->SetIsCustomRowReadOnlyDelegate(FIsCustomRowReadOnly::CreateLambda([this](FName InRowName, FName InParentName) { return bPreview; }));
	return GraphInspectorPanelBox;
}


void SInterchangeGraphInspectorWindow::Construct(const FArguments& InArgs)
{
	InterchangeBaseNodeContainer = InArgs._InterchangeBaseNodeContainer;
	bPreview = InArgs._bPreview;
	OwnerWindow = InArgs._OwnerWindow;

	check(InterchangeBaseNodeContainer != nullptr);
	check(OwnerWindow.IsValid());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SInterchangeGraphInspectorWindow::SortingNodes)

		InterchangeBaseNodeContainer->SortNodes(
			[](const UInterchangeBaseNode& LHS, const UInterchangeBaseNode& RHS)
			{
				const bool bLeftIsFactory = LHS.IsA<UInterchangeFactoryBaseNode>();
				const bool bRightIsFactory = RHS.IsA<UInterchangeFactoryBaseNode>();
				const bool bLeftIsScene = LHS.IsA<UInterchangeSceneNode>();
				const bool bRightIsScene = RHS.IsA<UInterchangeSceneNode>();

				// Always put scene nodes above other nodes
				if (bLeftIsScene != bRightIsScene)
				{
					return bLeftIsScene > bRightIsScene;
				}
				// Then prefer other nodes above factory nodes
				else if (bLeftIsFactory != bRightIsFactory)
				{
					return bLeftIsFactory < bRightIsFactory;
				}
				else
				{
					return LHS.GetDisplayLabel() < RHS.GetDisplayLabel();
				}
			}
		);
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(10.0f, 3.0f))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2)
			[
				SpawnGraphInspector()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("InspectorGraphWindow_Close", "Dismiss"))
				.OnClicked(this, &SInterchangeGraphInspectorWindow::OnCloseDialog)
				.Visibility(bPreview ? EVisibility::Collapsed : EVisibility::Visible)
			]
		]
	];
}

void SInterchangeGraphInspectorWindow::OnSelectionChanged(UInterchangeBaseNode* Item, ESelectInfo::Type SelectionType)
{
	//Change the object point by the InspectorBox
	GraphInspectorDetailsView->SetObject(Item);
}

void SInterchangeGraphInspectorWindow::CloseGraphInspector()
{
	InterchangeBaseNodeContainer = nullptr;
	GraphInspectorTreeview = nullptr;
	GraphInspectorDetailsView = nullptr;

	if (TSharedPtr<SWindow> OwnerWindowPin = OwnerWindow.Pin())
	{
		OwnerWindowPin->RequestDestroyWindow();
	}
	OwnerWindow = nullptr;
}

#undef LOCTEXT_NAMESPACE
