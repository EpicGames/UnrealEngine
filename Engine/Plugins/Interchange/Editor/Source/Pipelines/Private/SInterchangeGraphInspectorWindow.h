// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNode.h" // IWYU pragma: keep
#include "Widgets/Views/STreeView.h"

class SBox;
class UInterchangeBaseNode;

class IDetailsView;
class UInterchangeBaseNodeContainer;

DECLARE_DELEGATE_TwoParams(FOnGraphInspectorSelectionChanged, UInterchangeBaseNode*, ESelectInfo::Type)

class SInterchangeGraphInspectorTreeView : public STreeView<UInterchangeBaseNode*>
{
public:
	~SInterchangeGraphInspectorTreeView();

	SLATE_BEGIN_ARGS(SInterchangeGraphInspectorTreeView)
		: _InterchangeBaseNodeContainer(nullptr)
		, _bPreview(false)
		, _OnSelectionChangedDelegate()
		{}
		SLATE_ARGUMENT(UInterchangeBaseNodeContainer*, InterchangeBaseNodeContainer)
		SLATE_ARGUMENT(bool, bPreview)
		SLATE_EVENT(FOnGraphInspectorSelectionChanged, OnSelectionChangedDelegate)
	SLATE_END_ARGS()

		/** Construct this widget */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowGraphInspectorTreeView(UInterchangeBaseNode* Item, const TSharedRef< STableViewBase >& OwnerTable);
	void OnGetChildrenGraphInspectorTreeView(UInterchangeBaseNode* InParent, TArray< UInterchangeBaseNode* >& OutChildren);

	void OnToggleSelectAll(ECheckBoxState CheckType);
	FReply OnExpandAll();
	FReply OnCollapseAll();
protected:
	UInterchangeBaseNodeContainer* InterchangeBaseNodeContainer = nullptr;
	bool bPreview = false;
	/** Delegate to invoke when selection changes. */
	FOnGraphInspectorSelectionChanged OnSelectionChangedDelegate;
	/** the elements we show in the tree view */
	TArray<UInterchangeBaseNode*> RootNodeArray;

	/** Open a context menu for the current selection */
	TSharedPtr<SWidget> OnOpenContextMenu();
	void RecursiveSetImport(UInterchangeBaseNode* Node, bool bState);
	void RecursiveSetExpand(UInterchangeBaseNode* Node, bool ExpandState);
	void AddSelectionToImport();
	void RemoveSelectionFromImport();
	void SetSelectionImportState(bool MarkForImport);
	void OnTreeViewSelectionChanged(UInterchangeBaseNode* Item, ESelectInfo::Type SelectionType);
};

class SInterchangeGraphInspectorWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInterchangeGraphInspectorWindow)
		: _InterchangeBaseNodeContainer(nullptr)
		, _bPreview(false)
		, _OwnerWindow()
		{}

		SLATE_ARGUMENT( UInterchangeBaseNodeContainer*, InterchangeBaseNodeContainer)
		SLATE_ARGUMENT(bool, bPreview)
		SLATE_ARGUMENT(TWeakPtr<SWindow>, OwnerWindow)
	SLATE_END_ARGS()

public:

	SInterchangeGraphInspectorWindow();
	~SInterchangeGraphInspectorWindow();

	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	void CloseGraphInspector();

	FReply OnCloseDialog()
	{
		CloseGraphInspector();
		return FReply::Handled();
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnCloseDialog();
		}

		return FReply::Unhandled();
	}

private:
	TWeakPtr< SWindow > OwnerWindow;

	TSharedRef<SBox> SpawnGraphInspector();
	void OnSelectionChanged(UInterchangeBaseNode* Item, ESelectInfo::Type SelectionType);

	//SInterchangeGraphInspectorWindow Arguments
	
	UInterchangeBaseNodeContainer* InterchangeBaseNodeContainer;
	bool bPreview = false;
	
	
	//Graph Inspector UI elements

	TSharedPtr<SInterchangeGraphInspectorTreeView> GraphInspectorTreeview;
	TSharedPtr<IDetailsView> GraphInspectorDetailsView;
};
