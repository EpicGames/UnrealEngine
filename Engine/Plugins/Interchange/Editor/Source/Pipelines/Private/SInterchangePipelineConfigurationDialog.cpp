// Copyright Epic Games, Inc. All Rights Reserved.
#include "SInterchangePipelineConfigurationDialog.h"

#include "InterchangeEditorPipelineDetails.h"

#include "DetailsViewArgs.h"
#include "Dialog/SCustomDialog.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Views/TableViewMetadata.h"
#include "GameFramework/Actor.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "InterchangeCardsPipeline.h"
#include "InterchangeEditorPipelineStyle.h"
#include "InterchangeManager.h"
#include "InterchangePipelineConfigurationBase.h"
#include "InterchangeProjectSettings.h"
#include "InterchangeTranslatorBase.h"
#include "Interfaces/IMainFrameModule.h"
#include "Layout/Visibility.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "PropertyEditorModule.h"
#include "SInterchangeGraphInspectorWindow.h"
#include "SInterchangeTranslatorSettingsDialog.h"
#include "SPrimaryButton.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "InterchangePipelineConfiguration"

static bool GInterchangeDefaultShowEssentialsView = false;
static FAutoConsoleVariableRef CCvarInterchangeDefaultShowEssentialsView(
	TEXT("Interchange.FeatureFlags.Import.DefaultShowEssentialsView"),
	GInterchangeDefaultShowEssentialsView,
	TEXT("Whether the import dialog starts by default in essential pipeline properties layout."),
	ECVF_Default);

static bool GInterchangeDefaultShowSettings = false;
static FAutoConsoleVariableRef CCvarInterchangeDefaultShowSettings(
	TEXT("Interchange.FeatureFlags.Import.DefaultShowSettingsView"),
	GInterchangeDefaultShowSettings,
	TEXT("Whether the import dialog shows the settings by default. Settings mode is always shown if GInterchangeDefaultHideCardsView is true."),
	ECVF_Default);

static bool GInterchangeDefaultHideCardsView = true;
static FAutoConsoleVariableRef CCvarInterchangeDefaultHideCardsView(
	TEXT("Interchange.FeatureFlags.Import.DefaultHideCardsView"),
	GInterchangeDefaultHideCardsView,
	TEXT("Whether the import dialog should hide the basic cards view."),
	ECVF_Default);

static bool GInterchangeShowConflictWarningsOnCardsView = true;
static FAutoConsoleVariableRef CCvarInterchangeShowConflictWarningOnCardsView(
	TEXT("Interchange.FeatureFlags.Import.ShowConflictWarningsOnCardsView"),
	GInterchangeShowConflictWarningsOnCardsView,
	TEXT("Whether the import conflict warnings will be shown on cards view."),
	ECVF_Default);

constexpr double AdvancedUIRatio = 1.25;

const FName ReimportStackName = TEXT("ReimportPipeline");

// Pipeline Stack Preset Items
void SInterchangePipelineStackPresetItem::Construct(const FArguments& InArgs)
{
	PresetItem = InArgs._PresetItem;

	ensure(PresetItem.IsValid());
	AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(2.0, 0.0, 8.0, 0.0)
	[
		SNew(SBox)
		.WidthOverride(14.0f)
		.HeightOverride(14.0f)
		[
			SNew(SImage)
			.Image_Static(&SInterchangePipelineStackPresetItem::GetItemIconBrush)
		]
	];

	AddSlot()
	.VAlign(VAlign_Center)
	.FillWidth(1.0f)
	[
		SNew(STextBlock)
		.Text(FText::FromString(*PresetItem))
	];
}

const FSlateBrush* SInterchangePipelineStackPresetItem::GetItemIconBrush()
{
	static const FName IconName = FName("PipelineConfigurationIcon.PipelineStackPreset");
	const FSlateIcon SlateIcon = FSlateIconFinder::FindIcon(IconName);
	return SlateIcon.GetOptionalIcon();
}

// Pipeline List View Items

void SInterchangePipelineItem::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& OwnerTable,
	TSharedPtr<FInterchangePipelineItemType> InPipelineElement)
{
	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	PipelineElement = InPipelineElement;
	TObjectPtr<UInterchangePipelineBase> PipelineElementPtr = PipelineElement->Pipeline;
	check(PipelineElementPtr.Get());
	FText PipelineName = LOCTEXT("InvalidPipelineName", "Invalid Pipeline");
	if (PipelineElementPtr.Get())
	{
		FString PipelineNameString = PipelineElement->DisplayName;
		if (!PipelineElement->bShowEssentials)
		{
			PipelineNameString += FString::Printf(TEXT(" (%s)"), *PipelineElementPtr->GetClass()->GetName());
		}
		PipelineName = FText::FromString(PipelineNameString);
	}
	
	static const FSlateBrush* ConflictBrush = FAppStyle::GetBrush("Icons.Error");
	const FText ConflictsComboBoxTooltip = LOCTEXT("ConflictsComboBoxTooltip", "If there is some conflict, simply select one to see more details.");
	const FText Conflict_IconTooltip = FText::Format(LOCTEXT("Conflict_IconTooltip", "There are {0} conflicts. See Conflicts section below for details."), PipelineElement->ConflictInfos.Num());

	STableRow<TSharedPtr<FInterchangePipelineItemType>>::Construct(
		STableRow<TSharedPtr<FInterchangePipelineItemType>>::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 2.0f, 6.0f, 2.0f)
			[
				SNew(SImage)
				.Image(this, &SInterchangePipelineItem::GetImageItemIcon)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 2.0f, 6.0f, 2.0f)
			[
				SNew(SImage)
				.ToolTipText(Conflict_IconTooltip)
				.Image(ConflictBrush)
				.Visibility_Lambda([this]()->EVisibility
					{
						return PipelineElement->ConflictInfos.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
					})
				.ColorAndOpacity(this, &SInterchangePipelineItem::GetTextColor)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(3.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(PipelineName)
				.ColorAndOpacity(this, &SInterchangePipelineItem::GetTextColor)
			]
		], OwnerTable);
}

const FSlateBrush* SInterchangePipelineItem::GetImageItemIcon() const
{
	const FSlateBrush* TypeIcon = nullptr;
	FName IconName = "PipelineConfigurationIcon.Pipeline";
	const FSlateIcon SlateIcon = FSlateIconFinder::FindIcon(IconName);
	TypeIcon = SlateIcon.GetOptionalIcon();
	if (!TypeIcon)
	{
		TypeIcon = FSlateIconFinder::FindIconBrushForClass(AActor::StaticClass());
	}
	return TypeIcon;
}

FSlateColor SInterchangePipelineItem::GetTextColor() const
{
	if (PipelineElement->ConflictInfos.Num() > 0)
	{
		return FStyleColors::Warning;
	}
	return FSlateColor::UseForeground();
}

/************************************************************************/
/* SInterchangePipelineConfigurationDialog Implementation                      */
/************************************************************************/

namespace UE::Private
{
	bool ContainStack(const TArray<FInterchangeStackInfo>& PipelineStacks, const FName StackName)
	{
		return PipelineStacks.FindByPredicate([StackName](const FInterchangeStackInfo& StackInfo)
			{
				return (StackInfo.StackName == StackName);
			}) != nullptr;
	}
}

SInterchangePipelineConfigurationDialog::SInterchangePipelineConfigurationDialog()
{
	PipelineConfigurationDetailsView = nullptr;
	OwnerWindow = nullptr;
}

SInterchangePipelineConfigurationDialog::~SInterchangePipelineConfigurationDialog()
{
	if (TSharedPtr<SWindow> OwnerWindowPinned = OwnerWindow.Pin())
	{
		OwnerWindowPinned->GetOnWindowClosedEvent().RemoveAll(this);
	}

	if (PreviewNodeContainer.IsValid())
	{
		PreviewNodeContainer->ClearFlags(RF_Standalone | RF_Public);
		PreviewNodeContainer->ClearInternalFlags(EInternalObjectFlags::Async);
		PreviewNodeContainer.Reset();
	}
}

// Pipelines are renamed with the reimport prefix to avoid conflicts with the duplicates of the original pipelines that end up in the same package.
 // As this is the name displayed in the Dialog, conflicts won't matter.
FString SInterchangePipelineConfigurationDialog::GetPipelineDisplayName(const UInterchangePipelineBase* Pipeline)
{

	FString PipelineDisplayName = Pipeline->ScriptedGetPipelineDisplayName();
	if(PipelineDisplayName.IsEmpty())
	{
		PipelineDisplayName = Pipeline->GetName();
	}

	return PipelineDisplayName;
}

void SInterchangePipelineConfigurationDialog::SetEditPipeline(FInterchangePipelineItemType* PipelineItemToEdit)
{
	TArray<UObject*> ObjectsToEdit;
	ObjectsToEdit.Add(!PipelineItemToEdit ? nullptr : PipelineItemToEdit->Pipeline);

	if (PipelineItemToEdit)
	{
		PipelineItemToEdit->ConflictInfos.Reset();
		PipelineItemToEdit->ConflictInfos = PipelineItemToEdit->Pipeline->GetConflictInfos(PipelineItemToEdit->ReimportObject, PipelineItemToEdit->Container, PipelineItemToEdit->SourceData);
		FInterchangePipelineBaseDetailsCustomization::SetConflictsInfo(PipelineItemToEdit->ConflictInfos);

		//Acquire ExtraInformation from SourceNode and pass it to FInterchangePipelineBaseDetailsCustomization:
		TMap<FString, FString> ExtraInformation;
		const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(PipelineItemToEdit->Container);
		if (SourceNode)
		{
			SourceNode->GetExtraInformation(ExtraInformation);
		}
		FInterchangePipelineBaseDetailsCustomization::SetExtraInformation(ExtraInformation);
	}
	PipelineConfigurationDetailsView->SetObjects(ObjectsToEdit);
}

FReply SInterchangePipelineConfigurationDialog::OnEditTranslatorSettings()
{
	if (!TranslatorSettings || !Translator.IsValid())
	{
		return FReply::Handled();
	}

	TSharedRef<SInterchangeTranslatorSettingsDialog> OptionsDialog =
		SNew(SInterchangeTranslatorSettingsDialog)
		.WindowArguments(SWindow::FArguments()
			.IsTopmostWindow(false)
			.MinWidth(500)
			.MinHeight(400)
			.ClientSize(FVector2f{500, 400})
			.SizingRule(ESizingRule::UserSized))
		.TranslatorSettings(TranslatorSettings);


	OptionsDialog->GetTranslatorSettingsDialogClosed().BindLambda([this](bool bUserResponse, bool bTranslatorSettingsChanged)
		{
			if (!bTranslatorSettingsChanged)
			{
				return;
			}

			if (UClass* TranslatorSettingsClass = TranslatorSettings->GetClass())
			{
				//Save the config locally before the translation.
				TranslatorSettings->SaveSettings();

				//Need to Translate the source data
				FScopedSlowTask Progress(2.f, NSLOCTEXT("SInterchangePipelineConfigurationDialog", "TranslatingSourceFile...", "Translating source file..."));
				Progress.MakeDialog();
				Progress.EnterProgressFrame(1.f);
				//Reset the container
				BaseNodeContainer->Reset();

				Translator->Translate(*BaseNodeContainer.Get());

				//Refresh the dialog
				UpdateStack(CurrentStackName);

				Progress.EnterProgressFrame(1.f);
			}
		});

	OptionsDialog->ShowModal();

	return FReply::Handled();
}

void SInterchangePipelineConfigurationDialog::GatherConflictAndExtraInfo(TArray<FInterchangeConflictInfo>& ConflictInfo, TMap<FString, FString>& ExtraInfo)
{
	if (CurrentSelectedPipelineItem.IsValid())
	{
		TSharedPtr<FInterchangePipelineItemType> PipelineItem = CurrentSelectedPipelineItem.Pin();
		ConflictInfo = PipelineItem->Pipeline->GetConflictInfos(PipelineItem->ReimportObject, PipelineItem->Container, PipelineItem->SourceData);

		const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(PipelineItem->Container);
		if (SourceNode)
		{
			SourceNode->GetExtraInformation(ExtraInfo);
		}
	}
}

void SInterchangePipelineConfigurationDialog::OnExtendPipelineDetailsViewSettingsMenu(FMenuBuilder& MenuBuilder)
{
	FSlateIcon DummyIcon(NAME_None, NAME_None);

	MenuBuilder.BeginSection("Interchange", LOCTEXT("InterchangeMenuExtenderHeading", "Interchange"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowEssentialsOnly_Label", "Show Only Essential Properties"),
		LOCTEXT("ShowEssentialsOnly_Tooltip", "Display only the essential pipeline properties according to the pipeline settings"),
		DummyIcon,
		FUIAction(
			FExecuteAction::CreateSP(this, &SInterchangePipelineConfigurationDialog::Execute_ToggleShowEssentials),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() {return bShowEssentials; })
		),
		FName("PipelineShowEssentials"),
		EUserInterfaceActionType::Check
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterOnContent_Label", "Filter Based on Import Contents"),
		LOCTEXT("FilterOnContent_Tooltip", "Display properties based on the import source content"),
		DummyIcon,
		FUIAction(
			FExecuteAction::CreateSP(this, &SInterchangePipelineConfigurationDialog::Execute_ToggleFilterOnContent),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() {return bFilterOptions; })
		),
		FName("PipelineShowEssentials"),
		EUserInterfaceActionType::Check
	);
	MenuBuilder.EndSection();
}

void SInterchangePipelineConfigurationDialog::Execute_ToggleShowEssentials()
{
	bShowEssentials = !bShowEssentials;

	//Refresh the pipeline
	UpdateStack(CurrentStackName);
	GConfig->SetBool(TEXT("InterchangeImportDialogOptions"), TEXT("ShowEssentials"), bShowEssentials, GEditorPerProjectIni);
}

void SInterchangePipelineConfigurationDialog::Execute_ToggleFilterOnContent()
{
	bFilterOptions = !bFilterOptions;
	//Refresh the pipeline
	UpdateStack(CurrentStackName);
	GConfig->SetBool(TEXT("InterchangeImportDialogOptions"), TEXT("FilterOptions"), bFilterOptions, GEditorPerProjectIni);
}

TSharedRef<SBox> SInterchangePipelineConfigurationDialog::SpawnPipelineConfiguration()
{
	const ISlateStyle* InterchangeEditorPipelineStyle = FSlateStyleRegistry::FindSlateStyle("InterchangeEditorPipelineStyle");

	AvailableStacks.Reset();
	TSharedPtr<FString> SelectedStack;
	if (bReimport)
	{
		CurrentStackName = ReimportStackName;
	}
	else
	{
		CurrentStackName = FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(bSceneImport, *SourceData);
	}

	//In case we do not have a valid stack name use the first stack
	FName FirstStackName = PipelineStacks.Num() > 0 ? PipelineStacks[0].StackName : CurrentStackName;
	if (bTestConfigDialog || !UE::Private::ContainStack(PipelineStacks, CurrentStackName))
	{
		CurrentStackName = FirstStackName;
	}
	for(FInterchangeStackInfo& Stack : PipelineStacks)
	{
		TSharedPtr<FString> StackNamePtr = MakeShared<FString>(Stack.StackName.ToString());
		if (CurrentStackName == Stack.StackName)
		{
			for (const TObjectPtr<UInterchangePipelineBase>& DefaultPipeline : Stack.Pipelines)
			{
				check(DefaultPipeline);
				if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(DefaultPipeline))
				{
					GeneratedPipeline->TransferAdjustSettings(DefaultPipeline);
					if (GeneratedPipeline->IsFromReimportOrOverride())
					{
						//We save the pipeline settings to allow Reset to Default to work
						GeneratedPipeline->SaveSettings(Stack.StackName);
					}
					else
					{
						constexpr bool bResetPreDialogTrue = true;
						//Load the settings for this pipeline
						GeneratedPipeline->LoadSettings(Stack.StackName, bResetPreDialogTrue);
						GeneratedPipeline->PreDialogCleanup(Stack.StackName);
					}
					GeneratedPipeline->SetShowEssentialsMode(bShowEssentials);
					if (bFilterOptions && BaseNodeContainer.IsValid())
					{
						GeneratedPipeline->FilterPropertiesFromTranslatedData(BaseNodeContainer.Get());
					}
					PipelineListViewItems.Add(MakeShareable(new FInterchangePipelineItemType{ GetPipelineDisplayName(DefaultPipeline), GeneratedPipeline, ReimportObject.Get(), BaseNodeContainer.Get(), SourceData.Get(), bShowEssentials }));
				}
			}
			SelectedStack = StackNamePtr;
		}
		AvailableStacks.Add(StackNamePtr);
	}

	FText PipelineListTooltip = LOCTEXT("PipelineListTooltip", "Select a pipeline you want to edit properties for. The pipeline properties will be recorded and changes will be available in subsequent use of that pipeline");
	PipelinesListView = SNew(SPipelineListViewType)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(&PipelineListViewItems)
		.OnGenerateRow(this, &SInterchangePipelineConfigurationDialog::MakePipelineListRowWidget)
		.OnSelectionChanged(this, &SInterchangePipelineConfigurationDialog::OnPipelineSelectionChanged)
		.ClearSelectionOnClick(false)
		.ToolTipText(PipelineListTooltip);

	TSharedPtr<SPipelinePresetComboBox> PresetComboBoxPtr;
	//Only use a combo box if there is more then one stack
	if (AvailableStacks.Num() > 0)
	{
		FText StackComboBoxTooltip = LOCTEXT("StackComboBoxTooltip", "Selected pipeline stack preset will be used for the current import. See the Interchange project settings to modify the pipeline stacks preset list.");
		PresetComboBoxPtr = SNew(SPipelinePresetComboBox)
			.OptionsSource(&AvailableStacks)
			.OnGenerateWidget(this, &SInterchangePipelineConfigurationDialog::MakePipelinePresetComboBoxEntryWidget)
			.OnSelectionChanged(this, &SInterchangePipelineConfigurationDialog::OnStackSelectionChanged)
			.ContentPadding(FMargin(2.0f, 4.0f))
			.ToolTipText(StackComboBoxTooltip)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0, 0.0, 8.0, 0.0)
				[
					SNew(SBox)
					.WidthOverride(14.0f)
					.HeightOverride(14.0f)
					[
						SNew(SImage)
						.Image(SInterchangePipelineStackPresetItem::GetItemIconBrush())
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Center)
				.Padding(0.0, 0.0, 8.0, 0.0)
				[
					SNew(STextBlock)
					.Text(this, &SInterchangePipelineConfigurationDialog::GetSelectedPipelineStackPresetLabelText)
				]
			];

		if (SelectedStack.IsValid())
		{
			PresetComboBoxPtr->SetSelectedItem(SelectedStack);
		}
	}

	FText CurrentStackText = LOCTEXT("CurrentStackText", "Pipeline Stack Preset");

	TSharedPtr<SWidget> StackTextComboBox;
	if (!PresetComboBoxPtr.IsValid())
	{
		CurrentStackText = LOCTEXT("CurrentStackTextNoComboBox", "There is no pipeline stack preset available");
		StackTextComboBox = SNew(SBox)
		[
			SNew(STextBlock)
			.Text(CurrentStackText)
		];
	}
	else
	{
		StackTextComboBox = SNew(SBox)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SBox)
				[
					SNew(STextBlock)
					.Text(CurrentStackText)
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				PresetComboBoxPtr.ToSharedRef()
			]
		];
	}

	TSharedPtr<SWidget> StackAndGroupWidget;

	//Groups
	FText GroupUsedText = LOCTEXT("GroupUsedText", "Group Used:");

	FInterchangeGroup::EUsedGroupStatus UsedGroupStatus;
	const FInterchangeGroup& UsedInterchangeGroup = FInterchangeProjectSettingsUtils::GetUsedGroup(UsedGroupStatus);

	switch (UsedGroupStatus)
	{
		case FInterchangeGroup::NotSet:
			StackAndGroupWidget = SNew(SBox)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							StackTextComboBox.ToSharedRef()
						]
				];
			break;
		case FInterchangeGroup::SetAndValid:
			{
				FText GroupComboBoxTooltip = LOCTEXT("GroupComboBoxTooltip", "Group usage can be set in Editor Preferences > Interchange > Groups.");

				StackAndGroupWidget = SNew(SBox)
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								StackTextComboBox.ToSharedRef()
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(SBox)
									[
										SNew(SHorizontalBox)
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.Padding(16.0f, 0.0f, 4.0f, 0.0f)
											.AutoWidth()
											[
												SNew(SBox)
													[
														SNew(STextBlock)
															.Text(GroupUsedText)
													]
											]
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(SEditableTextBox)
													.Text(FText::FromName(UsedInterchangeGroup.DisplayName))
													.IsEnabled(false)
													.ToolTipText(GroupComboBoxTooltip)
											]
									]
							]
					];
			}
			break;
		case FInterchangeGroup::SetAndInvalid:
			{
				//invalid Group usage:
				FText InvalidGroupText = LOCTEXT("InvalidGroupText", "Invalid Group setup for usage!");
				FText InvalidGroupTooltip = LOCTEXT("InvalidGroupTooltip", "Please review Group usage in Editor Preferences > Interchange > Groups.");

				StackAndGroupWidget = SNew(SBox)
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								StackTextComboBox.ToSharedRef()
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.VAlign(VAlign_Center)
									.Padding(16.0f, 0.0f, 0.0f, 0.0f)
									.AutoWidth()
									[
										SNew(SBox)
											[
												SNew(STextBlock)
													.Text(InvalidGroupText)
													.ToolTipText(InvalidGroupTooltip)
											]
									]
							]
					];
			}
			break;
		default:
			break;
	}

	TSharedPtr<SBox> InspectorBox;

	FText FilterPipelineTooltip = LOCTEXT("SInterchangePipelineConfigurationDialog_FilterPipelineOptions_tooltip", "Filter the pipeline options using the source content data.");
	FText EssentialPipelineTooltip = LOCTEXT("SInterchangePipelineConfigurationDialog_ShowEssentialsOptions_tooltip", "Display only essentials pipeline properties.");

	TSharedRef<SBox> PipelineConfigurationPanelBox = SNew(SBox)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f, 8.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					StackAndGroupWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNullWidget::NullWidget
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.IsEnabled_Lambda([this]()
						{
							return PipelinesListView->GetNumItemsSelected() == 1;
						})
					.Text(LOCTEXT("SInterchangePipelineConfigurationDialog_ResetToPipelineAsset", "Use Pipeline Defaults"))
					.ToolTipText_Lambda([bReimportClosure = bReimport]()
						{
							if (bReimportClosure)
							{
								return LOCTEXT("SInterchangePipelineConfigurationDialog_ResetToPipelineAsset_TooltipReimport", "Reset the selected pipeline to is values used the last time this asset was imported.");
							}
							else
							{
								return LOCTEXT("SInterchangePipelineConfigurationDialog_ResetToPipelineAsset_Tooltip", "Reset the selected pipeline to is default values.");
							}
						})
					.OnClicked(this, &SInterchangePipelineConfigurationDialog::OnResetToDefault)
				]
			]
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Visibility_Lambda([this]()
						{
							if (!CurrentSelectedStackPresetItem.IsValid())
							{
								return EVisibility::Collapsed;
							}

							return EVisibility::Visible;
						})
					.Text_Lambda([this]()
						{
							if (!CurrentSelectedStackPresetItem.IsValid())
							{
								return FText();
							}

							return FText::Format(LOCTEXT("PipelineListViewTitleFormat", "Pipelines in {0} pipeline stack:"), GetSelectedPipelineStackPresetLabelText());
						})
			]
			+ SVerticalBox::Slot()
			.Padding(0.0f, 8.0f)
			.AutoHeight()
			[
				SNew(SBox)
				.MinDesiredHeight(50)
				.MaxDesiredHeight(140)
				[
					PipelinesListView.ToSharedRef()
				]
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(InspectorBox, SBox)
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowSectionSelector = true;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bShowKeyablePropertiesOption = false;
	DetailsViewArgs.bShowAnimatedPropertiesOption = false;
	DetailsViewArgs.bShowHiddenPropertiesWhilePlayingOption = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = FName("InterchangeImportDialog");
	
	DetailsViewArgs.OptionsExtender = MakeShareable(new FExtender);
	DetailsViewArgs.OptionsExtender->AddMenuExtension(
		"DetailsViewShowOptions",
		EExtensionHook::Before,
		nullptr, 
		FMenuExtensionDelegate::CreateSP(this, &SInterchangePipelineConfigurationDialog::OnExtendPipelineDetailsViewSettingsMenu)
	);
	
	
	PipelineConfigurationDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	// Reset to 'All' Sections on spawning the modal only. Subsequent changes to details view objects will use the persisted section selection.
	bPendingDetailsViewSectionReset = true;
	PipelineConfigurationDetailsView->SetOnObjectArrayChanged(FOnObjectArrayChanged::CreateLambda(
		[this](const FString& /*Title*/, const TArray<UObject*>& /*SelectedObjects*/)
		{
			if (bPendingDetailsViewSectionReset)
			{
				bPendingDetailsViewSectionReset = !PipelineConfigurationDetailsView->ResetToDefaultSection();
			}
		})
	);

	InspectorBox->SetContent(PipelineConfigurationDetailsView->AsShared());
	SetEditPipeline(nullptr);
	PipelineConfigurationDetailsView->GetIsPropertyVisibleDelegate().BindLambda([this](const FPropertyAndParent& PropertyAndParent)
		{
			return IsPropertyVisible(PropertyAndParent);
		});
	PipelineConfigurationDetailsView->OnFinishedChangingProperties().AddLambda([this](const FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (CurrentSelectedPipeline && CurrentSelectedPipeline->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
		{
			//Refresh the pipeline
			UpdateStack(CurrentStackName);
		}
	});
	return PipelineConfigurationPanelBox;
}

TSharedRef<SBox> SInterchangePipelineConfigurationDialog::SpawnCardsConfiguration()
{
	const FSlateBrush* AdvanceSettingsIcon = FSlateIconFinder::FindIcon("PipelineConfigurationIcon.SidePanelRight").GetOptionalIcon();

	CreateCardsViewList();

	TSharedRef<SWidget> BodyWidget = CardViewList.IsValid() ? CardViewList.ToSharedRef() : SNullWidget::NullWidget;
	TSharedRef<SBox> CardsConfigurationPanelBox = SNew(SBox)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.Padding(0.0f, 8.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SInterchangePipelineConfigurationDialog_AssetFoundText", "Assets Found in Source:"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.Padding(FMargin(8.0, 4.0, 0.0, 4.0))
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
				.Type(ESlateCheckBoxType::ToggleButton)
				.IsChecked_Lambda([this]() { return bShowSettings ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckState)
					{
						bShowSettings = CheckState == ECheckBoxState::Checked;

						TSharedPtr<SWindow> DialogWindow = OwnerWindow.Pin();
						FVector2D ClientSize = DialogWindow->GetClientSizeInScreen();
						FWindowSizeLimits SizeLimits = DialogWindow->GetSizeLimits();
						double MinimumSizeX = 0.0;
						if (bShowCards)
						{
							//Add show cards width
							MinimumSizeX += OriginalMinWindowSize;
						}
						if (bShowSettings)
						{
							//Add settings width
							MinimumSizeX += (OriginalMinWindowSize * AdvancedUIRatio);
						}

						//Resize the client with a updated minimum size width
						SizeLimits.SetMinWidth(static_cast<float>(MinimumSizeX));
						DialogWindow->SetSizeLimits(SizeLimits);
						DialogWindow->Resize(ClientSize);

						UpdateStack(CurrentStackName);
						GConfig->SetBool(TEXT("InterchangeImportDialogOptions"), TEXT("ShowSettings"), bShowSettings, GEditorPerProjectIni);
					})
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SImage)
						.Image(AdvanceSettingsIcon)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SInterchangePipelineConfigurationDialog_AdvanceSettingsButtonText", "Advanced Settings"))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0.0f, 8.0f)
		.FillHeight(1.0f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				BodyWidget
			]
		]
	];
	return CardsConfigurationPanelBox;
}

void SInterchangePipelineConfigurationDialog::Construct(const FArguments& InArgs)
{
	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		GEditor->Trans->SetUndoBarrier();
	}


	//Make sure there is a valid default value

	OwnerWindow = InArgs._OwnerWindow;
	SourceData = InArgs._SourceData;
	bSceneImport = InArgs._bSceneImport;
	bReimport = InArgs._bReimport;
	bTestConfigDialog = InArgs._bTestConfigDialog;
	PipelineStacks = InArgs._PipelineStacks;
	OutPipelines = InArgs._OutPipelines;
	BaseNodeContainer = InArgs._BaseNodeContainer;
	ReimportObject = InArgs._ReimportObject;
	SourceData = InArgs._SourceData;
	Translator = InArgs._Translator;

	if (Translator.IsValid())
	{
		TranslatorSettings = Translator->GetSettings();
	}

	if (ReimportObject.IsValid())
	{
		ensure(bReimport);
	}
	
	FText ReuseSettingsTooltipText = LOCTEXT("InspectorGraphWindow_ReuseSettingsTooltipText", "When importing multiple files this checkbox allow users to use the same settings for source of the same extension.");
	const bool bTranslatorThreadSafe = Translator.IsValid() ? Translator->IsThreadSafe() : false;
	if (!bTranslatorThreadSafe)
	{
		FString Extension = SourceData.IsValid() ? FPaths::GetExtension(SourceData->GetFilename()) : TEXT("N/A");
		ReuseSettingsTooltipText = FText::Format(LOCTEXT("InspectorGraphWindow_ReuseSettingsNotThreadSafeTooltipText", "{0} translator is not thread safe and must use the same settings for subsequent files"), FText::FromString(Extension));
	}

	check(OutPipelines);

	check(OwnerWindow.IsValid());
	TSharedPtr<SWindow> OwnerWindowPinned = OwnerWindow.Pin();
	if (OwnerWindowPinned.IsValid())
	{
		OwnerWindowPinned->GetOnWindowClosedEvent().AddRaw(this, &SInterchangePipelineConfigurationDialog::OnWindowClosed);
		OriginalMinWindowSize = OwnerWindowPinned->GetSizeLimits().GetMinWidth().Get(0.0);
		if (OriginalMinWindowSize < 1.0)
		{
			OriginalMinWindowSize = 550.0;
		}
		DeltaClientWindowSize = (OwnerWindowPinned->GetSizeInScreen() - OwnerWindowPinned->GetClientSizeInScreen()).X;
	}

	//Get the default layout when the user open the import dialog for the first time.
	bShowEssentials = GInterchangeDefaultShowEssentialsView;
	bShowCards = !GInterchangeDefaultHideCardsView && !bSceneImport;

	bShowSettings = GInterchangeDefaultShowSettings || !bShowCards;

	if (bReimport)
	{
		bFilterOptions = false;
	}
	
	if(GConfig->DoesSectionExist(TEXT("InterchangeImportDialogOptions"), GEditorPerProjectIni))
	{
		if (!bReimport)
		{
			GConfig->GetBool(TEXT("InterchangeImportDialogOptions"), TEXT("FilterOptions"), bFilterOptions, GEditorPerProjectIni);
		}
		GConfig->GetBool(TEXT("InterchangeImportDialogOptions"), TEXT("ShowEssentials"), bShowEssentials, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("InterchangeImportDialogOptions"), TEXT("ShowSettings"), bShowSettings, GEditorPerProjectIni);
		//Make sure settings are shown if we hide cards
		if(!bShowCards)
		{
			bShowSettings = true;
		}
		GConfig->GetDouble(TEXT("InterchangeImportDialogOptions"), TEXT("SplitAdvancedRatio"), SplitAdvancedRatio, GEditorPerProjectIni);
	}

	if (InArgs._bOverrideDefaultShowEssentials.IsSet())
	{
		bShowEssentials = InArgs._bOverrideDefaultShowEssentials.GetValue();
	}

	if (InArgs._bOverrideDefaultFilterOnContent.IsSet())
	{
		bFilterOptions = InArgs._bOverrideDefaultFilterOnContent.GetValue();
	}

	//Make sure the windows is width enough to show all the ui part (cards and settings)
	if (OwnerWindowPinned.IsValid())
	{
		FVector2D WidowsClientSize = OwnerWindowPinned->GetClientSizeInScreen();
		FWindowSizeLimits SizeLimits = OwnerWindowPinned->GetSizeLimits();
		double MinimumSizeX = 0.0;
		if (bShowCards)
		{
			//Add show cards width
			MinimumSizeX += OriginalMinWindowSize;
		}
		if (bShowSettings)
		{
			//Add settings width
			MinimumSizeX += (OriginalMinWindowSize * AdvancedUIRatio);
		}
		SizeLimits.SetMinWidth(static_cast<float>(MinimumSizeX));
		OwnerWindowPinned->SetSizeLimits(SizeLimits);
		//Resize the window to respect the limits
		OwnerWindowPinned->Resize(WidowsClientSize);
	}

	//SpawnPipelineConfiguration must always be call because it create the pipeline list from the project settings
	TSharedRef<SBox> MainBodyAdvanced = SpawnPipelineConfiguration();
	UpdatePipelineSupportedAssetClasses();
	TSharedRef<SBox> MainBodyCardsConfiguration = SpawnCardsConfiguration();

	const FSlateBrush* TranslatorSettingsIcon = FSlateIconFinder::FindIcon("PipelineConfigurationIcon.TranslatorSettings").GetOptionalIcon();

	const ISlateStyle* InterchangeEditorPipelineStyle = FSlateStyleRegistry::FindSlateStyle("InterchangeEditorPipelineStyle");

	const FSlateBrush* ImportSourceBorderBrush = nullptr;
	if (InterchangeEditorPipelineStyle)
	{
		ImportSourceBorderBrush = InterchangeEditorPipelineStyle->GetBrush("ImportSource.Dropdown.Border");
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(16.0f, 16.0f))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 0.0f, 0.0f, 8.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SInterchangePipelineConfigurationDialog_SourceLabel", "Import Source"))
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(ImportSourceBorderBrush)
					.Padding(0.0f, 4.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						.Padding(8.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(this, &SInterchangePipelineConfigurationDialog::GetSourceDescription)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SSeparator)
							.Orientation(EOrientation::Orient_Vertical)
							.Thickness(1.0f)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("NoBorder"))
							.ContentPadding(FMargin(0.0f))
							.Visibility_Lambda([this]()
								{
									return !TranslatorSettings ? EVisibility::Collapsed : EVisibility::Visible;
								})
							.ToolTipText(LOCTEXT("SInterchangePipelineConfigurationDialog_TranslatorSettings_Tooltip", "Edit translator project settings."))
							.OnClicked(this, &SInterchangePipelineConfigurationDialog::OnEditTranslatorSettings)
							[
								SNew(SImage)
								.Image(TranslatorSettingsIcon)
							]
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(CardsAndAdvancedSplitter, SSplitter)
				.OnSplitterFinishedResizing_Lambda([this]()
					{
						SplitAdvancedRatio = (1.0 - static_cast<double>(CardsAndAdvancedSplitter->SlotAt(0).GetSizeValue()));
						GConfig->SetDouble(TEXT("InterchangeImportDialogOptions"), TEXT("SplitAdvancedRatio"), SplitAdvancedRatio, GEditorPerProjectIni);
					})
				+ SSplitter::Slot()
				.MinSize(static_cast<float>(OriginalMinWindowSize - DeltaClientWindowSize))
				.Value(1.0f - static_cast<float>(SplitAdvancedRatio))
				[
					SNew(SBox)
					.Visibility_Lambda([this]()
						{
							return bShowCards ? EVisibility::Visible : EVisibility::Collapsed;
						})
					.Padding_Lambda([this]()
						{
							return (bShowSettings && bShowCards) ? FMargin(0.0f, 0.0f, 8.0f, 0.0f) : FMargin(0.0f, 0.0f, 0.0f, 0.0f);
						})
					[
						MainBodyCardsConfiguration
					]
				]
				+ SSplitter::Slot()
				.MinSize(static_cast<float>((OriginalMinWindowSize * AdvancedUIRatio) - DeltaClientWindowSize))
				.Value(static_cast<float>(SplitAdvancedRatio))
				[
					SNew(SBox)
					.Visibility_Lambda([this]()
						{
							return bShowSettings ? EVisibility::Visible : EVisibility::Collapsed;
						})
					.Padding_Lambda([this]()
						{
							return bShowCards ? FMargin(8.0f, 0.0f, 0.0f, 0.0f) : FMargin(0.0f, 0.0f, 0.0f, 0.0f);
						})
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						MainBodyAdvanced
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0, 0.0, 0.0)
			[
				SNew(SSeparator)
				.Orientation(EOrientation::Orient_Horizontal)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.f, 0.f)
				.AutoWidth()
				[
					IDocumentation::Get()->CreateAnchor(FString("interchange-framework-in-unreal-engine"))
				]
				+ SHorizontalBox::Slot()
				.Padding(4.f, 0.f)
				.AutoWidth()
				[
					SNew(SHorizontalBox)
					.ToolTipText(LOCTEXT("InspectorGraphWindow_ReuseSettingsToolTip", "When importing multiple files, keep the same import settings for every file or open the settings dialog for each file."))
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(4.f, 0.f)
					[
						SNew(STextBlock)
						.IsEnabled(bTranslatorThreadSafe)
						.Text(LOCTEXT("InspectorGraphWindow_ReuseSettings", "Use the same settings for subsequent files"))
						.ToolTipText(ReuseSettingsTooltipText)
					]
					+ SHorizontalBox::Slot()
					.Padding(4.f, 0.f)
					[
						SAssignNew(UseSameSettingsForAllCheckBox, SCheckBox)
						.IsChecked(true)
						.IsEnabled_Lambda([this, bTranslatorThreadSafe]()
							{
								if (!bTranslatorThreadSafe)
								{
									return false;
								}
								return IsImportButtonEnabled();
							})
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNullWidget::NullWidget
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FMargin(4.f, 0.f))
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SPrimaryButton)
						.Icon(this, &SInterchangePipelineConfigurationDialog::GetImportButtonIcon)
						.Text_Lambda([this]() 
							{
								return bTestConfigDialog ? LOCTEXT("InspectorGraphWindow_SaveConfig", "Save Config") : LOCTEXT("InspectorGraphWindow_Import", "Import");
							})
						.ToolTipText(this, &SInterchangePipelineConfigurationDialog::GetImportButtonTooltip)
						.IsEnabled(this, &SInterchangePipelineConfigurationDialog::IsImportButtonEnabled)
						.OnClicked(this, &SInterchangePipelineConfigurationDialog::OnCloseDialog, ECloseEventType::PrimaryButton)
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton)
						.Visibility_Lambda([this]()
							{
								return bShowSettings ? EVisibility::Visible : EVisibility::Collapsed;
							})
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("InspectorGraphWindow_Preview", "Preview..."))
						.IsEnabled_Lambda([this]() 
							{
								return !bTestConfigDialog;  
							})
						.OnClicked(this, &SInterchangePipelineConfigurationDialog::OnPreviewImport)
					]
					+SUniformGridPanel::Slot(2,0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("InspectorGraphWindow_Cancel", "Cancel"))
						.OnClicked(this, &SInterchangePipelineConfigurationDialog::OnCloseDialog, ECloseEventType::Cancel)
					]
				]
			]
		]
	];

	//Select the first pipeline
	if (PipelineListViewItems.Num() > 0)
	{
		bool bSelectFirst = true;
		FString LastPipelineName;
		FString KeyName = CurrentStackName.ToString() + TEXT("_LastSelectedPipeline");
		if (GConfig->GetString(TEXT("InterchangeSelectPipeline"), *KeyName, LastPipelineName, GEditorPerProjectIni))
		{
			for (TSharedPtr<FInterchangePipelineItemType> PipelineItem : PipelineListViewItems)
			{
				FString PipelineItemName = PipelineItem->Pipeline->GetClass()->GetName();
				if (PipelineItemName.Equals(LastPipelineName))
				{
					PipelinesListView->SetSelection(PipelineItem, ESelectInfo::Direct);
					bSelectFirst = false;
					break;
				}
			}
		}
		if (bSelectFirst)
		{
			PipelinesListView->SetSelection(PipelineListViewItems[0], ESelectInfo::Direct);
		}

		if (bShowCards && GInterchangeShowConflictWarningsOnCardsView)
		{
			RefreshCardsViewList();
		}
	}

	FInterchangePipelineBaseDetailsCustomization::OnGatherConflictAndExtraInfo.BindRaw(this, &SInterchangePipelineConfigurationDialog::GatherConflictAndExtraInfo);
}

bool SInterchangePipelineConfigurationDialog::IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	if (bReimport)
	{
		const FName ReimportRestrictKey(TEXT("ReimportRestrict"));
		return !(PropertyAndParent.Property.GetBoolMetaData(ReimportRestrictKey));
	}
	return true;
}

const FSlateBrush* SInterchangePipelineConfigurationDialog::GetImportButtonIcon() const
{
	const FSlateBrush* TypeIcon = nullptr;
	if (bShowSettings || GInterchangeShowConflictWarningsOnCardsView)
	{
		for (TSharedPtr<FInterchangePipelineItemType> PipelineItem : PipelineListViewItems)
		{
			if (PipelineItem.IsValid() && PipelineItem->Pipeline)
			{
				if (PipelineItem->ConflictInfos.Num() > 0)
				{
					const FSlateIcon SlateIcon = FSlateIconFinder::FindIcon("Icons.Warning");
					return SlateIcon.GetOptionalIcon();
				}
			}
		}
	}
	return TypeIcon;
}


FText SInterchangePipelineConfigurationDialog::GetSourceDescription() const
{
	FText ActionDescription;
	if (SourceData.IsValid())
	{
		ActionDescription = FText::FromString(SourceData->GetFilename());
	}
	return ActionDescription;
}

FReply SInterchangePipelineConfigurationDialog::OnResetToDefault()
{
	FReply Result = FReply::Handled();
	TArray<TWeakObjectPtr<UObject>> SelectedPipelines = PipelineConfigurationDetailsView->GetSelectedObjects();
	if (CurrentStackName == NAME_None)
	{
		return Result;
	}

	FInterchangePipelineItemType* PipelineToEdit = nullptr;

	//Multi selection is not allowed
	for(TWeakObjectPtr<UObject> WeakObject : SelectedPipelines)
	{
		//We test the cast because we can have null or other type selected (i.e. translator settings class default object).
		if (UInterchangePipelineBase* Pipeline = Cast<UInterchangePipelineBase>(WeakObject.Get()))
		{
			const UClass* PipelineClass = Pipeline->GetClass();

			for (FInterchangeStackInfo& Stack : PipelineStacks)
			{
				if (Stack.StackName != CurrentStackName)
				{
					continue;
				}
				for (const TObjectPtr<UInterchangePipelineBase>& DefaultPipeline : Stack.Pipelines)
				{
					//We assume the pipelines inside one stack are all different classes, we use the class to know which default asset we need to duplicate
					if (DefaultPipeline->GetClass() == PipelineClass)
					{
						for(int32 PipelineIndex = 0; PipelineIndex < PipelineListViewItems.Num(); ++PipelineIndex)
						{
							TObjectPtr<UInterchangePipelineBase> PipelineElement = PipelineListViewItems[PipelineIndex]->Pipeline;
							if (PipelineElement.Get() == Pipeline)
							{
								if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(DefaultPipeline))
								{
									GeneratedPipeline->TransferAdjustSettings(DefaultPipeline);
									GeneratedPipeline->SetShowEssentialsMode(bShowEssentials);
									if(bFilterOptions && BaseNodeContainer.IsValid())
									{
										GeneratedPipeline->FilterPropertiesFromTranslatedData(BaseNodeContainer.Get());
									}
									//Switch the pipeline the element point on
									PipelineListViewItems[PipelineIndex]->Pipeline = GeneratedPipeline;
									PipelineToEdit = PipelineListViewItems[PipelineIndex].Get();
									PipelinesListView->SetSelection(PipelineListViewItems[PipelineIndex], ESelectInfo::Direct);
									PipelinesListView->RequestListRefresh();
									break;
								}
							}
						}
					}
				}
			}
		}
	}
	SetEditPipeline(PipelineToEdit);

	//Update the cards
	RefreshCardsViewList();

	return Result;
}

bool SInterchangePipelineConfigurationDialog::ValidateAllPipelineSettings(TOptional<FText>& OutInvalidReason) const
{
	for (TSharedPtr<FInterchangePipelineItemType> PipelineElement : PipelineListViewItems)
	{
		check(PipelineElement->Pipeline);
		if (!PipelineElement->Pipeline->IsSettingsAreValid(OutInvalidReason))
		{
			return false;
		}
	}
	return true;
}

bool SInterchangePipelineConfigurationDialog::IsImportButtonEnabled() const
{
	TOptional<FText> InvalidReason;
	return ValidateAllPipelineSettings(InvalidReason);
}

FText SInterchangePipelineConfigurationDialog::GetImportButtonTooltip() const
{
	//Pipeline validation
	TOptional<FText> InvalidReason;
	if (!ValidateAllPipelineSettings(InvalidReason) && InvalidReason.IsSet())
	{
		return InvalidReason.GetValue();
	}

	//Pipeline conflicts
	for (TSharedPtr<FInterchangePipelineItemType> PipelineItem : PipelineListViewItems)
	{
		if (PipelineItem.IsValid() && PipelineItem->Pipeline)
		{
			if (PipelineItem->ConflictInfos.Num() > 0)
			{
				return LOCTEXT("ImportButtonConflictTooltip", "There is one or more pipeline conflicts, look at any conflict in the pipeline list to have more detail.");
			}
		}
	}

	//Default tooltip
	return LOCTEXT("ImportButtonDefaultTooltip", "Selected pipeline stack will be used for the current import");
}

void SInterchangePipelineConfigurationDialog::SaveAllPipelineSettings() const
{
	for (TSharedPtr<FInterchangePipelineItemType> PipelineElement : PipelineListViewItems)
	{
		if (PipelineElement->Pipeline)
		{
			PipelineElement->Pipeline->SaveSettings(CurrentStackName);
		}
	}
}

void SInterchangePipelineConfigurationDialog::ClosePipelineConfiguration(const ECloseEventType CloseEventType)
{
	if (CloseEventType == ECloseEventType::Cancel || CloseEventType == ECloseEventType::WindowClosing)
	{
		bCanceled = true;
		bImportAll = false;
	}
	else //ECloseEventType::PrimaryButton
	{
		bCanceled = false;
		bImportAll = UseSameSettingsForAllCheckBox->IsChecked();
		
		//Fill the OutPipelines array
		for (TSharedPtr<FInterchangePipelineItemType> PipelineElement : PipelineListViewItems)
		{
			OutPipelines->Add(PipelineElement->Pipeline);
		}
		if (UInterchangeCardsPipeline* InterchangeCardsPipeline = GenerateTransientCardsPipeline())
		{
			//Add the cards pipeline if valid
			OutPipelines->Add(InterchangeCardsPipeline);
		}
	}

	//Save the settings only if its not a re-import
	if (!bReimport)
	{
		SaveAllPipelineSettings();
	}

	PipelineConfigurationDetailsView = nullptr;

	if (CloseEventType != ECloseEventType::WindowClosing)
	{
		if (TSharedPtr<SWindow> OwnerWindowPin = OwnerWindow.Pin())
		{
			OwnerWindowPin->GetOnWindowClosedEvent().RemoveAll(this);
			OwnerWindowPin->RequestDestroyWindow();
		}
	}
	OwnerWindow = nullptr;

	FInterchangePipelineBaseDetailsCustomization::OnGatherConflictAndExtraInfo.Unbind();

	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		GEditor->Trans->RemoveUndoBarrier();
	}
}

FReply SInterchangePipelineConfigurationDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCloseDialog(ECloseEventType::Cancel);
	}
	return FReply::Unhandled();
}

void SInterchangePipelineConfigurationDialog::UpdateStack(const FName& NewStackName)
{
	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	const bool bStackSelectionChange = CurrentStackName != NewStackName;

	//Save current stack settings, we want the same settings when we will go back to the same stack
	//When doing a reimport we do not want to save the setting because the context have special default
	//value for some options like: (Import Materials, Import Textures...).
	//So when doing a reimport switching stack is like doing a reset to default on all pipelines
	if (!bReimport || !bStackSelectionChange)
	{
		SaveAllPipelineSettings();
	}
	CurrentStackName = NewStackName;

	int32 CurrentPipelineIndex = 0;
	if (!bStackSelectionChange)
	{
		//store the selected pipeline
		for (int32 PipelineIndex = 0; PipelineIndex < PipelineListViewItems.Num(); ++PipelineIndex)
		{
			const TSharedPtr<FInterchangePipelineItemType> PipelineItem = PipelineListViewItems[PipelineIndex];
			if (PipelineItem->Pipeline == CurrentSelectedPipeline)
			{
				CurrentPipelineIndex = PipelineIndex;
				break;
			}
		}
	}

	//Rebuild the Pipeline list item
	PipelineListViewItems.Reset();

	for (FInterchangeStackInfo& Stack : PipelineStacks)
	{
		TSharedPtr<FString> StackNamePtr = MakeShared<FString>(Stack.StackName.ToString());
		if (CurrentStackName != Stack.StackName)
		{
			continue;
		}
		for (const TObjectPtr<UInterchangePipelineBase>& DefaultPipeline : Stack.Pipelines)
		{
			check(DefaultPipeline);
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(DefaultPipeline))
			{
				GeneratedPipeline->TransferAdjustSettings(DefaultPipeline);
				if (!GeneratedPipeline->IsFromReimportOrOverride() || !bStackSelectionChange)
				{
					//Load the settings for this pipeline
					GeneratedPipeline->LoadSettings(Stack.StackName, bStackSelectionChange);
					if (bStackSelectionChange)
					{
						//Do not reset pipeline value if we are just refreshing the filtering
						GeneratedPipeline->PreDialogCleanup(Stack.StackName);
					}
				}
				GeneratedPipeline->SetShowEssentialsMode(bShowEssentials);
				if (bFilterOptions && BaseNodeContainer.IsValid())
				{
					GeneratedPipeline->FilterPropertiesFromTranslatedData(BaseNodeContainer.Get());
				}
				PipelineListViewItems.Add(MakeShareable(new FInterchangePipelineItemType{ GetPipelineDisplayName(DefaultPipeline), GeneratedPipeline, ReimportObject.Get(), BaseNodeContainer.Get(), SourceData.Get(), bShowEssentials }));
			}
		}
	}
	//Select the first pipeline
	if (PipelineListViewItems.Num() > 0)
	{
		CurrentPipelineIndex = PipelineListViewItems.IsValidIndex(CurrentPipelineIndex) ? CurrentPipelineIndex : 0;
		if (bShowSettings)
		{
			PipelinesListView->SetSelection(PipelineListViewItems[CurrentPipelineIndex], ESelectInfo::Direct);
			PipelinesListView->RequestListRefresh();
		}
		else
		{
			for (TSharedPtr<FInterchangePipelineItemType>& PiplineListViewItem : PipelineListViewItems)
			{
				PiplineListViewItem->ConflictInfos.Reset();
				PiplineListViewItem->ConflictInfos = PiplineListViewItem->Pipeline->GetConflictInfos(PiplineListViewItem->ReimportObject, PiplineListViewItem->Container, PiplineListViewItem->SourceData);
			}
		}
	}

	//Update the cards
	RefreshCardsViewList();
}

FText SInterchangePipelineConfigurationDialog::GetSelectedPipelineStackPresetLabelText() const
{
	if (CurrentSelectedStackPresetItem.IsValid())
	{
		return FText::FromString(*(CurrentSelectedStackPresetItem.Pin()));
	}
	return LOCTEXT("NoSelectedStackPreset", "No Selected Stack Preset");
}

TSharedRef<SWidget> SInterchangePipelineConfigurationDialog::MakePipelinePresetComboBoxEntryWidget(TSharedPtr<FString> InPresetItem)
{
	return SNew(SInterchangePipelineStackPresetItem)
				.PresetItem(InPresetItem);
}

void SInterchangePipelineConfigurationDialog::OnStackSelectionChanged(TSharedPtr<FString> String, ESelectInfo::Type)
{
	if (!String.IsValid())
	{
		return;
	}
	
	CurrentSelectedStackPresetItem = String.ToWeakPtr();

	FName NewStackName = FName(*String.Get());
	if (!UE::Private::ContainStack(PipelineStacks, NewStackName))
	{
		return;
	}

	//Nothing change the selection is the same
	if (CurrentStackName == NewStackName)
	{
		return;
	}

	UpdateStack(NewStackName);
}

TSharedRef<ITableRow> SInterchangePipelineConfigurationDialog::MakePipelineListRowWidget(
	TSharedPtr<FInterchangePipelineItemType> InElement,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InElement->Pipeline);
	return SNew(SInterchangePipelineItem, OwnerTable, InElement);
}

void SInterchangePipelineConfigurationDialog::OnPipelineSelectionChanged(TSharedPtr<FInterchangePipelineItemType> InItem, ESelectInfo::Type SelectInfo)
{
	CurrentSelectedPipeline = nullptr;
	if (InItem)
	{
		CurrentSelectedPipeline = InItem->Pipeline;
	}
	CurrentSelectedPipelineItem = InItem;
	SetEditPipeline(InItem.Get());
	
	if (CurrentSelectedPipeline)
	{
		FString CurrentPipelineName = CurrentSelectedPipeline->GetClass()->GetName();
		FString KeyName = CurrentStackName.ToString() + TEXT("_LastSelectedPipeline");
		GConfig->SetString(TEXT("InterchangeSelectPipeline"), *KeyName, *CurrentPipelineName, GEditorPerProjectIni);
	}
}
void SInterchangePipelineConfigurationDialog::UpdatePipelineSupportedAssetClasses()
{
	PipelineSupportAssetClasses.Reset();
	for (TSharedPtr<FInterchangePipelineItemType> PipelineItem : PipelineListViewItems)
	{
		TArray<UClass*> PipelineSupportedClasses;
		PipelineItem->Pipeline->GetSupportAssetClasses(PipelineSupportedClasses);
		for (UClass* AssetClass : PipelineSupportedClasses)
		{
			if (!PipelineSupportAssetClasses.Contains(AssetClass))
			{
				PipelineSupportAssetClasses.Add(AssetClass);
			}
		}
	}

	PipelineSupportedAssetData.Empty();
	for (UClass* SupportedClass : PipelineSupportAssetClasses)
	{
		PipelineSupportedAssetData.Emplace(SupportedClass, TSet<UClass*>());
	}
}

void SInterchangePipelineConfigurationDialog::UpdateEnableDataPerFactoryNodeClass()
{
	if (!ensure(PreviewNodeContainer.IsValid()))
	{
		return;
	}

	TArray<UClass*> ValidCardClasses;
	PreviewNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([this, &ValidCardClasses](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
		{
			UClass* NodeObjectClass = FactoryNode->GetObjectClass();
			// Don't add factory node class twice.
			// Don't add factory node class that don't have a valid object class
			// Don't add factory node class that are not a main asset from pipelines
			if (!ValidCardClasses.Contains(FactoryNode->GetClass())
				&& NodeObjectClass)
			{
				bool bClassIsSupported = false;
				UClass* AssetClass = nullptr;
				for (UClass* SupportedClass : PipelineSupportAssetClasses)
				{
					if (NodeObjectClass->IsChildOf(SupportedClass))
					{
						AssetClass = SupportedClass;
						bClassIsSupported = true;
						break;
					}
				}
				if (bClassIsSupported)
				{
					ValidCardClasses.Add(FactoryNode->GetClass());
					PipelineSupportedAssetData[AssetClass].FactoryNodeClasses.Add(FactoryNode->GetClass());
					
					if (!EnableDataPerFactoryNodeClass.Contains(FactoryNode->GetClass()))
					{
						//Add a new entry that is enabled by default
						EnableDataPerFactoryNodeClass.FindOrAdd(FactoryNode->GetClass()).ObjectClass = FactoryNode->GetObjectClass();
					}
				}
			}
		});

	//Remove class card that do not exist anymore
	TArray<UClass*> CardsClassesToRemove;
	for (TPair<UClass*, FFactoryNodeEnabledData>& FactoryNodeClassAndEnableStatus : EnableDataPerFactoryNodeClass)
	{
		if (!ValidCardClasses.Contains(FactoryNodeClassAndEnableStatus.Key))
		{
			CardsClassesToRemove.Add(FactoryNodeClassAndEnableStatus.Key);
		}
	}
	for (UClass* CardToRemove : CardsClassesToRemove)
	{
		EnableDataPerFactoryNodeClass.Remove(CardToRemove);
	}
}

void SInterchangePipelineConfigurationDialog::FillAssetCardsList()
{
	//Update the pipeline supported asset class
	UpdatePipelineSupportedAssetClasses();

	//Update the preview container
	constexpr bool bUpdateCardsTrue = true;
	UpdatePreviewContainer(bUpdateCardsTrue);
	if (ensure(PreviewNodeContainer.IsValid()))
	{
		UpdateEnableDataPerFactoryNodeClass();
		AssetCards.Reset();
		for (TPair<UClass*, FPerSupportedAssetClassData>& SupportedAssetDataPair : PipelineSupportedAssetData)
		{
			UClass* AssetClass = SupportedAssetDataPair.Key;
			FPerSupportedAssetClassData AssetClassData = SupportedAssetDataPair.Value;

			TSharedPtr<SInterchangeAssetCard> AssetCard;
			if (AssetClassData.FactoryNodeClasses.IsEmpty())
			{
				AssetCard = SNew(SInterchangeAssetCard)
					.PreviewNodeContainer(PreviewNodeContainer.Get())
					.AssetClass(AssetClass)
					.CardDisabled(true)
					.ShouldImportAssetType_Lambda([](){return false;})
					.OnImportAssetTypeChanged_Lambda([](bool bNewEnabledValue){});
			}
			else
			{
				for (UClass* FactoryNodeClass : AssetClassData.FactoryNodeClasses)
				{
					AssetClass = EnableDataPerFactoryNodeClass.FindChecked(FactoryNodeClass).ObjectClass;
					AssetCard = SNew(SInterchangeAssetCard)
						.PreviewNodeContainer(PreviewNodeContainer.Get())
						.AssetClass(AssetClass)
						.CardDisabled(false)
						.ShouldImportAssetType_Lambda([this, FactoryNodeClass]()
							{
								return EnableDataPerFactoryNodeClass.FindChecked(FactoryNodeClass).bEnable;
							})
						.OnImportAssetTypeChanged_Lambda([this, FactoryNodeClass](bool bNewEnabledValue)
							{
								EnableDataPerFactoryNodeClass.FindChecked(FactoryNodeClass).bEnable = bNewEnabledValue;
							});

					if (GInterchangeShowConflictWarningsOnCardsView
						&& PipelineListViewItems.Num() > 0)
					{
						for (TSharedPtr<FInterchangePipelineItemType>& PipelineListViewItem : PipelineListViewItems)
						{
							if (AssetCard->RefreshHasConflicts(PipelineListViewItem->ConflictInfos))
							{
								break;
							}
						}
					}
				}
			}

			if (AssetCard.IsValid())
			{
				AssetCards.Add(AssetCard);
			}
		}
	}
}

void SInterchangePipelineConfigurationDialog::CreateCardsViewList()
{
	if (!bShowCards)
	{
		return;
	}

	FillAssetCardsList();
	if (AssetCards.IsEmpty())
	{
		CardViewList = nullptr;
	}
	else
	{
		CardViewList = SNew(SInterchangeAssetCardList).AssetCards(&AssetCards);
	}
}

void SInterchangePipelineConfigurationDialog::RefreshCardsViewList()
{
	if (!bShowCards)
	{
		return;
	}

	FillAssetCardsList();
	if (CardViewList.IsValid())
	{
		CardViewList->RefreshList(PreviewNodeContainer.Get());
	}
}

UInterchangeCardsPipeline* SInterchangePipelineConfigurationDialog::GenerateTransientCardsPipeline() const
{
	UInterchangeCardsPipeline* InterchangeCardsPipeline = nullptr;
	if (!bReimport)
	{
		TArray<UClass*> DisabledNodeClasses;
		for (const TPair<UClass*, FFactoryNodeEnabledData>& FactoryNodeClassAndEnableStatus : EnableDataPerFactoryNodeClass)
		{
			if(!FactoryNodeClassAndEnableStatus.Value.bEnable)
			{
				DisabledNodeClasses.Add(FactoryNodeClassAndEnableStatus.Key);
			}
		}

		if (!DisabledNodeClasses.IsEmpty())
		{
			InterchangeCardsPipeline = NewObject<UInterchangeCardsPipeline>();
			InterchangeCardsPipeline->SetDisabledFactoryNodes(DisabledNodeClasses);
		}
	}
	return InterchangeCardsPipeline;
}

void SInterchangePipelineConfigurationDialog::UpdatePreviewContainer(bool bUpdateCards) const
{
	auto ClearObjectFlags = [](UObject* Obj)
		{
			Obj->ClearFlags(RF_Standalone | RF_Public);
			Obj->ClearInternalFlags(EInternalObjectFlags::Async);
		};
	if (PreviewNodeContainer.IsValid())
	{
		ClearObjectFlags(PreviewNodeContainer.Get());
		PreviewNodeContainer.Reset();
	}
	PreviewNodeContainer = DuplicateObject<UInterchangeBaseNodeContainer>(BaseNodeContainer.Get(), GetTransientPackage());
	PreviewNodeContainer->SetChildrenCache(BaseNodeContainer->GetChildrenCache());

	TArray<UInterchangeSourceData*> SourceDatas;
	SourceDatas.Add(SourceData.Get());

	FString ContentBasePath = FString();
	if (bReimport && ReimportObject.IsValid())
	{
		ContentBasePath = FPaths::GetPath(ReimportObject->GetPathName());
	}

	//Execute all pipelines on the duplicated container
	UInterchangeResultsContainer* Results = NewObject<UInterchangeResultsContainer>(GetTransientPackage());
	for (int32 PipelineIndex = 0; PipelineIndex < PipelineListViewItems.Num(); ++PipelineIndex)
	{
		const TSharedPtr<FInterchangePipelineItemType> PipelineItem = PipelineListViewItems[PipelineIndex];

		//Duplicate the pipeline because ScriptedExecutePipeline is not const
		if (UInterchangePipelineBase* DuplicatedPipeline = DuplicateObject<UInterchangePipelineBase>(PipelineItem->Pipeline, GetTransientPackage()))
		{
			DuplicatedPipeline->TransferAdjustSettings(PipelineItem->Pipeline);
			DuplicatedPipeline->SetResultsContainer(Results);
			DuplicatedPipeline->ScriptedExecutePipeline(PreviewNodeContainer.Get(), SourceDatas, ContentBasePath);
			ClearObjectFlags(DuplicatedPipeline);
		}
	}

	if (!bUpdateCards)
	{
		//If we do not update cards execute the cards pipeline since its a final preview
		if (UInterchangeCardsPipeline* InterchangeCardsPipeline = GenerateTransientCardsPipeline())
		{
			InterchangeCardsPipeline->ScriptedExecutePipeline(PreviewNodeContainer.Get(), SourceDatas, ContentBasePath);
		}
	}

	PreviewNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([ClosureReimportObject = ReimportObject](const FString& NodeUid, UInterchangeFactoryBaseNode* Node)
		{

			//Set all node in preview mode so hide the internal data attributes
			Node->UserInterfaceContext = EInterchangeNodeUserInterfaceContext::Preview;

			//If we reimport a specific object we want to disabled all factory nodes that are not supporting the reimport object class
			if (ClosureReimportObject.IsValid())
			{
				Node->SetEnabled(ClosureReimportObject.Get()->IsA(Node->GetObjectClass()));
			}
		});

	//Make sure all temporary object are not flags to persist
	ClearObjectFlags(Results);
}

FReply SInterchangePipelineConfigurationDialog::OnPreviewImport() const
{
	auto ClearObjectFlags = [](UObject* Obj)
		{
			Obj->ClearFlags(RF_Standalone | RF_Public);
			Obj->ClearInternalFlags(EInternalObjectFlags::Async);
		};

	constexpr bool bUpdateCardsFalse = false;
	UpdatePreviewContainer(bUpdateCardsFalse);

	if (!ensure(PreviewNodeContainer.IsValid()))
	{
		return FReply::Handled();
	}

	//Create and show the graph inspector UI dialog
	TSharedRef<SWindow> Window = SNew(SWindow)
		.ClientSize(FVector2D(800.f, 650.f))
		.Title(NSLOCTEXT("SInterchangePipelineConfigurationDialog", "InterchangePreviewTitle", "Interchange Preview"));
	TSharedPtr<SInterchangeGraphInspectorWindow> InterchangeGraphInspectorWindow;

	Window->SetContent
	(
		SAssignNew(InterchangeGraphInspectorWindow, SInterchangeGraphInspectorWindow)
		.InterchangeBaseNodeContainer(PreviewNodeContainer.Get())
		.bPreview(true)
		.OwnerWindow(Window)
	);

	FSlateApplication::Get().AddModalWindow(Window, OwnerWindow.Pin(), false);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
