// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangePipelineBase.h"
#include "InterchangePipelineConfigurationBase.h"
#include "SInterchangeAssetCard.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/STreeView.h"

class SBox;

struct FPropertyAndParent;
struct FSlateBrush;
class IDetailsView;
class SCheckBox;
class UInterchangeCardsPipeline;
class UInterchangeTranslatorBase;


class SInterchangePipelineConfigurationDialog;

// Pipeline Stack Presets
class SInterchangePipelineStackPresetItem : public SHorizontalBox
{
public:
	SLATE_BEGIN_ARGS(SInterchangePipelineStackPresetItem)
		: _PresetItem(nullptr)
		{}
		SLATE_ARGUMENT(TSharedPtr<FString>, PresetItem)		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	static const FSlateBrush* GetItemIconBrush();

private:
	TSharedPtr<FString> PresetItem = nullptr;
};

typedef SComboBox< TSharedPtr<FString> > SPipelinePresetComboBox;

// Pipeline List View 
struct FInterchangePipelineItemType
{
public:
	FString DisplayName;
	UInterchangePipelineBase* Pipeline;
	UObject* ReimportObject = nullptr;
	UInterchangeBaseNodeContainer* Container = nullptr;
	UInterchangeSourceData* SourceData = nullptr;
	bool bShowEssentials = false;
	TArray<FInterchangeConflictInfo> ConflictInfos;
};

class SInterchangePipelineItem : public STableRow<TSharedPtr<FInterchangePipelineItemType>>
{
public:
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<STableViewBase>& OwnerTable,
		TSharedPtr<FInterchangePipelineItemType> InPipelineElement);
private:
	const FSlateBrush* GetImageItemIcon() const;
	FSlateColor GetTextColor() const;

	TSharedPtr<FInterchangePipelineItemType> PipelineElement = nullptr;
};

typedef SListView< TSharedPtr<FInterchangePipelineItemType> > SPipelineListViewType;

enum class ECloseEventType : uint8
{
	Cancel,
	PrimaryButton,
	WindowClosing
};

class SInterchangePipelineConfigurationDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInterchangePipelineConfigurationDialog)
		: _OwnerWindow()
	{}

		SLATE_ARGUMENT(TWeakPtr<SWindow>, OwnerWindow)
		SLATE_ARGUMENT(TWeakObjectPtr<UInterchangeSourceData>, SourceData)
		SLATE_ARGUMENT(bool, bSceneImport)
		SLATE_ARGUMENT(bool, bReimport)
		SLATE_ARGUMENT(bool, bTestConfigDialog)
		SLATE_ARGUMENT(TArray<FInterchangeStackInfo>, PipelineStacks)
		SLATE_ARGUMENT(TArray<UInterchangePipelineBase*>*, OutPipelines)
		SLATE_ARGUMENT(TWeakObjectPtr<UInterchangeBaseNodeContainer>, BaseNodeContainer)
		SLATE_ARGUMENT(TWeakObjectPtr<UObject>, ReimportObject)
		SLATE_ARGUMENT(TWeakObjectPtr<UInterchangeTranslatorBase>, Translator)
		SLATE_ARGUMENT(TOptional<bool>, bOverrideDefaultShowEssentials)
		SLATE_ARGUMENT(TOptional<bool>, bOverrideDefaultFilterOnContent)
	SLATE_END_ARGS()

public:

	SInterchangePipelineConfigurationDialog();
	virtual ~SInterchangePipelineConfigurationDialog();

	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	void ClosePipelineConfiguration(const ECloseEventType CloseEventType);

	FReply OnCloseDialog(const ECloseEventType CloseEventType)
	{
		ClosePipelineConfiguration(CloseEventType);
		return FReply::Handled();
	}

	void OnWindowClosed(const TSharedRef<SWindow>& ClosedWindow)
	{
		ClosePipelineConfiguration(ECloseEventType::WindowClosing);
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	bool IsCanceled() const { return bCanceled; }
	bool IsImportAll() const { return bImportAll; }

private:
	TSharedRef<SBox> SpawnPipelineConfiguration();
	TSharedRef<SBox> SpawnCardsConfiguration();

	bool IsPropertyVisible(const FPropertyAndParent&) const;
	FText GetSourceDescription() const;
	FReply OnResetToDefault();

	bool ValidateAllPipelineSettings(TOptional<FText>& OutInvalidReason) const;
	bool IsImportButtonEnabled() const;
	FText GetImportButtonTooltip() const;

	void SaveAllPipelineSettings() const;

	/** Internal utility function to properly display pipeline's name */
	static FString GetPipelineDisplayName(const UInterchangePipelineBase* Pipeline);

	void SetEditPipeline(FInterchangePipelineItemType* PipelineItemToEdit);
	FReply OnEditTranslatorSettings();

	void GatherConflictAndExtraInfo(TArray<FInterchangeConflictInfo>& ConflictInfo, TMap<FString, FString>& ExtraInfo);

	// Details View Menu Extension
	void OnExtendPipelineDetailsViewSettingsMenu(FMenuBuilder& MenuBuilder);
	void Execute_ToggleShowEssentials();
	void Execute_ToggleFilterOnContent();

private:
	TWeakPtr< SWindow > OwnerWindow;
	double OriginalMinWindowSize = 0.0;
	double DeltaClientWindowSize = 0.0;
	TWeakObjectPtr<UInterchangeSourceData> SourceData;
	TWeakObjectPtr<UInterchangeBaseNodeContainer> BaseNodeContainer;
	mutable TWeakObjectPtr<UInterchangeBaseNodeContainer> PreviewNodeContainer;
	TWeakObjectPtr<UObject> ReimportObject;
	TWeakObjectPtr<UInterchangeTranslatorBase> Translator;
	TObjectPtr<UInterchangeTranslatorSettings> TranslatorSettings = nullptr;
	TArray<FInterchangeStackInfo> PipelineStacks;
	TArray<UInterchangePipelineBase*>* OutPipelines;

	// The available stacks
	TArray<TSharedPtr<FString>> AvailableStacks;

	TSharedRef<SWidget> MakePipelinePresetComboBoxEntryWidget(TSharedPtr<FString> InElement);
	FText GetSelectedPipelineStackPresetLabelText() const;

	void OnStackSelectionChanged(TSharedPtr<FString> String, ESelectInfo::Type);
	void UpdateStack(const FName& NewStackName);

	//////////////////////////////////////////////////////////////////////////
	// the pipelines list view
	
	TSharedPtr<SPipelineListViewType> PipelinesListView;
	TArray< TSharedPtr< FInterchangePipelineItemType > > PipelineListViewItems;

	/** list view generate row callback */
	TSharedRef<ITableRow> MakePipelineListRowWidget(TSharedPtr<FInterchangePipelineItemType> InElement, const TSharedRef<STableViewBase>& OwnerTable);

	void OnPipelineSelectionChanged(TSharedPtr<FInterchangePipelineItemType> InItem, ESelectInfo::Type SelectInfo);

	//
	//////////////////////////////////////////////////////////////////////////
	struct FFactoryNodeEnabledData
	{
		bool bEnable = true;
		TObjectPtr<UClass> ObjectClass = nullptr;
	};
	TArray<TObjectPtr<UClass>> PipelineSupportAssetClasses;

	struct FPerSupportedAssetClassData
	{
		TSet<UClass*> FactoryNodeClasses;
	};

	TMap<UClass*, FPerSupportedAssetClassData> PipelineSupportedAssetData;

	TSharedPtr<SScrollBar> CardViewScrollbar;
	TMap<UClass*, FFactoryNodeEnabledData> EnableDataPerFactoryNodeClass;
	TSharedPtr<SInterchangeAssetCardList> CardViewList = nullptr;
	TArray<TSharedPtr<SInterchangeAssetCard>> AssetCards;
	void UpdatePipelineSupportedAssetClasses();
	void UpdateEnableDataPerFactoryNodeClass();
	void FillAssetCardsList();
	void CreateCardsViewList();
	void RefreshCardsViewList();
	UInterchangeCardsPipeline* GenerateTransientCardsPipeline() const;

	//Splitter management
	double SplitAdvancedRatio = 0.6;
	TSharedPtr<SSplitter> CardsAndAdvancedSplitter = nullptr;

	const FSlateBrush* GetImportButtonIcon() const;

	void UpdatePreviewContainer(bool bUpdateCards) const;

	FReply OnPreviewImport() const;

	TSharedPtr<IDetailsView> PipelineConfigurationDetailsView;
	TSharedPtr<SCheckBox> UseSameSettingsForAllCheckBox;

	bool bSceneImport = false;
	bool bReimport = false;
	bool bCanceled = false;
	bool bImportAll = false;

	bool bPendingDetailsViewSectionReset = false;

	// This is used to denote that the import dialog is opened from the Interchange Test Plan asset to configure the chosen Pipelines.
	bool bTestConfigDialog = false;

	bool bFilterOptions = false;
	bool bShowEssentials = false;
	bool bShowSettings = false;
	bool bShowCards = true;

	FName CurrentStackName = NAME_None;
	TWeakPtr<FString> CurrentSelectedStackPresetItem = nullptr;
	TObjectPtr<UInterchangePipelineBase> CurrentSelectedPipeline = nullptr;
	TWeakPtr<FInterchangePipelineItemType> CurrentSelectedPipelineItem = nullptr;
};
