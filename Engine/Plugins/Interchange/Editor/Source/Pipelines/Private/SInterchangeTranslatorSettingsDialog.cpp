// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInterchangeTranslatorSettingsDialog.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "InterchangeTranslatorBase.h"

#define LOCTEXT_NAMESPACE "SInterchangeTranslatorSettingsDialog"

void SInterchangeTranslatorSettingsDialog::Construct(const FArguments& InArgs)
{
	TranslatorSettings = InArgs._TranslatorSettings;
	if (TranslatorSettings.IsValid())
	{
		TranslatorSettingsCDO = TranslatorSettings->GetClass()->GetDefaultObject<UInterchangeTranslatorSettings>();
		OriginalTranslatorSettings = DuplicateObject<UInterchangeTranslatorSettings>(TranslatorSettings.Get(), GetTransientPackage());
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowSectionSelector = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bShowKeyablePropertiesOption = false;
	DetailsViewArgs.bShowAnimatedPropertiesOption = false;
	DetailsViewArgs.bShowHiddenPropertiesWhilePlayingOption = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedRef<IDetailsView> TranslatorSettingsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	TranslatorSettingsDetailsView->SetObject(TranslatorSettings.Get());

	TranslatorSettingsDetailsView->OnFinishedChangingProperties().AddLambda([this](const FPropertyChangedEvent& PropertyChangedEvent)
		{
			bTranslatorSettingsChanged = true;
		});

	SWindow::Construct(SWindow::FArguments(InArgs._WindowArguments)
		.Title(LOCTEXT("TranslatorSettingsTitle", "Translator Settings"))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush( "ToolPanel.GroupBorder" ))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					TranslatorSettingsDetailsView
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
								.Text(LOCTEXT("TranslatorSettings_ResetToDefault", "Reset To Default"))
								.OnClicked_Lambda([this]()
									{
										if (TranslatorSettingsCDO)
										{
											//Save the CDO settings
											TranslatorSettingsCDO->SaveSettings();
											// Restore the CDO Settings
											TranslatorSettings.Get()->LoadSettings();
											bTranslatorSettingsChanged = true;
										}
										return FReply::Handled();
									})
						]
						// Buttons
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Right)
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
								.AutoWidth()
								[
									SNew(SButton)
										.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
										.Text(LOCTEXT("TranslatorSettings_OK", "OK"))
										.OnClicked_Lambda([this]() 
											{
												bUserResponse = true;
												RequestDestroyWindow();
												return FReply::Handled();
											})
								]
								+ SHorizontalBox::Slot()
								.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
								.AutoWidth()
								[
									SNew(SButton)
										.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
										.Text(LOCTEXT("TranslatorSettings_Cancel", "Cancel"))
										.OnClicked_Lambda([this]()
											{
												if (OriginalTranslatorSettings)
												{
													//Save the Original settings
													OriginalTranslatorSettings->SaveSettings();
													// Restore the Original Settings
													TranslatorSettings.Get()->LoadSettings();
													bTranslatorSettingsChanged = false;
												}

												bUserResponse = false;
												RequestDestroyWindow();
												return FReply::Handled();
											})
								]
						]
				]
			]
		]);
}

bool SInterchangeTranslatorSettingsDialog::ShowModal()
{
	FSlateApplication::Get().AddModalWindow(StaticCastSharedRef<SWindow>(this->AsShared()), FGlobalTabmanager::Get()->GetRootWindow());

	if (OnTranslatorSettingsDialogClosed.IsBound())
	{
		OnTranslatorSettingsDialogClosed.Execute(bUserResponse, bTranslatorSettingsChanged);
	}

	return bUserResponse;
}



#undef LOCTEXT_NAMESPACE