// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Widgets/SWindow.h"
#include "UObject/WeakObjectPtr.h"
#include "Delegates/DelegateCombinations.h"

class UInterchangeTranslatorSettings;

DECLARE_DELEGATE_TwoParams(FOnTranslatorSettingsDialogClosed, bool, bool);

class SInterchangeTranslatorSettingsDialog : public SWindow
{

public:
	SLATE_BEGIN_ARGS(SInterchangeTranslatorSettingsDialog)
		{
			_AccessibleParams = FAccessibleWidgetData(EAccessibleBehavior::Auto);
		}

		/*********** Functional ***********/
		/** Event triggered when the dialog is closed, either because one of the buttons is pressed, or the windows is closed. */
		SLATE_EVENT(FSimpleDelegate, OnClosed)

		/** Provides default values for SWindow::FArguments not overriden by SCustomDialog. */
		SLATE_ARGUMENT(SWindow::FArguments, WindowArguments)

		SLATE_ARGUMENT(TWeakObjectPtr<UInterchangeTranslatorSettings>, TranslatorSettings)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	/** Show a modal dialog. Will block until an input is received.
	 *  
	 */
	bool ShowModal();

	FOnTranslatorSettingsDialogClosed& GetTranslatorSettingsDialogClosed() { return OnTranslatorSettingsDialogClosed; }

private:
	FOnTranslatorSettingsDialogClosed OnTranslatorSettingsDialogClosed;

	TWeakObjectPtr<UInterchangeTranslatorSettings> TranslatorSettings;
	UInterchangeTranslatorSettings* OriginalTranslatorSettings = nullptr;
	UInterchangeTranslatorSettings* TranslatorSettingsCDO = nullptr;

	bool bTranslatorSettingsChanged = false;
	bool bUserResponse = false;

};
