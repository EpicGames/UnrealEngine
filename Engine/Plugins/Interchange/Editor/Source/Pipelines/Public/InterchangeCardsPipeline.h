// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "InterchangePipelineBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "InterchangeCardsPipeline.generated.h"

#define UE_API INTERCHANGEEDITORPIPELINES_API

class UInterchangeBaseNodeContainer;
class UInterchangeSourceData;
/**
 * This pipeline is use by the interchange default UI to enable and disable factory node.
 *
 * @note This is an import only pipeline that is execute only when importing from the interchange default dialog.
 */
UCLASS(MinimalAPI)
class UInterchangeCardsPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////
	// BEGIN Pre import pipeline properties


	// END Pre import pipeline properties
	//////////////////////////////////////////////////////////////////////////

	/*
	 * This pipeline must never be saved into any asset import data.
	 */
	virtual bool SupportReimport() const override { return false; }

	/*
	 * Set all factory node class to disabled.
	 */
	UE_API void SetDisabledFactoryNodes(TArray<UClass*> FactoryNodeClasses);

protected:

	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		//If a Blueprint or Python class derived from this class, it will be executed on the game thread because we cannot currently execute script outside of the game thread, even if this function return true.
		return true;
	}
private:

	UPROPERTY()
	TArray<TObjectPtr<UClass>> FactoryNodeClassesToDisabled;
};

#undef UE_API
