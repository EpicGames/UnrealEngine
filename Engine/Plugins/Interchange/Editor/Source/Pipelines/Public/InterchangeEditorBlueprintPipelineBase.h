// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "InterchangeBlueprintPipelineBase.h"
#include "InterchangePipelineBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"


#include "InterchangeEditorBlueprintPipelineBase.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeEditorPipelineBase : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	//UObject Interface
	virtual bool IsEditorOnly() const override
	{
		return true;
	}

	virtual UWorld* GetWorld() const override;
	// End of UObject interface
};

/**
 * This class allow users to create editor only Interchange blueprint pipeline.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeEditorBlueprintPipelineBase : public UInterchangeBlueprintPipelineBase
{
	GENERATED_BODY()
public:

	UInterchangeEditorBlueprintPipelineBase()
	{
		ParentClass = UInterchangeEditorPipelineBase::StaticClass();
		//We must make sure the GeneratedClass is generated after the blueprint is loaded
		bRecompileOnLoad = true;
	}

	//UObject Interface
	virtual bool IsEditorOnly() const override
	{
		return true;
	}
	// End of UObject interface

	// UBlueprint interface
	virtual bool SupportedByDefaultBlueprintFactory() const override
	{
		return false;
	}

	virtual bool AlwaysCompileOnLoad() const override
	{
		return true;
	}
	// End of UBlueprint interface
};
