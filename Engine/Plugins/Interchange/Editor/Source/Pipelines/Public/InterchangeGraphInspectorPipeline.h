// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"

#include "InterchangeGraphInspectorPipeline.generated.h"

#define UE_API INTERCHANGEEDITORPIPELINES_API

/**
 * This pipeline is the generic pipeline option for all types of meshes. It should be called before specialized mesh pipelines like the generic static mesh or skeletal mesh pipelines.
 * All import options that are shared between mesh types should be added here.
 *
 * UPROPERTY possible meta values:
 * @meta ImportOnly - Boolean. The property is used only for import, not for reimport. Cannot be mixed with ReimportOnly.
 * @meta ReimportOnly - Boolean. The property is used only for reimport, not for import. Cannot be mixed with ImportOnly.
 * @meta MeshType - String. The property is for static mesh or skeletal mesh or both. If not specified, it will apply to all mesh types.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGraphInspectorPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////
	// BEGIN Pre import pipeline properties


	// END Pre import pipeline properties
	//////////////////////////////////////////////////////////////////////////

	/*
	 * This pipeline must never be saved into any asset import data.
	 */
	virtual bool SupportReimport() const override { return false; }

protected:

	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;

	//virtual bool ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FName& NodeKey, UObject* CreatedAsset) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		//If a Blueprint or Python class derived from this class, it will be executed on the game thread because we cannot currently execute script outside of the game thread, even if this function return true.
		return false;
	}

	//virtual bool ExecuteExportPipeline(UInterchangeBaseNodeContainer* BaseNodeContainer) override;
private:

};

#undef UE_API
