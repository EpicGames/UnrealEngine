// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineConfigurationBase.h"

#include "InterchangePipelineConfigurationGeneric.generated.h"

#define UE_API INTERCHANGEEDITORPIPELINES_API

class UInterchangeBaseNodeContainer;
class UInterchangeSourceData;

class SWindow;
class SInterchangePipelineConfigurationDialog;

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangePipelineConfigurationGeneric : public UInterchangePipelineConfigurationBase
{
	GENERATED_BODY()

public:

protected:
	UE_API virtual EInterchangePipelineConfigurationDialogResult ShowPipelineDialog_Internal(FPipelineConfigurationDialogParams& InParams) override;

};

#undef UE_API
