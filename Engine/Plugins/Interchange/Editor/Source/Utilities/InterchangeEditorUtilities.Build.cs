// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InterchangeEditorUtilities : ModuleRules
	{
		public InterchangeEditorUtilities(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InterchangeCore",
					"InterchangeEngine"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DesktopPlatform",
					"Slate",
					"SlateCore",
					"UnrealEd"
				}
			);
		}
	}
}
