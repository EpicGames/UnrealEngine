// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeEditorUtilities.h"

#include "FileHelpers.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Editor/EditorEngine.h"

extern UNREALED_API UEditorEngine* GEditor;
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeEditorUtilities)

bool UInterchangeEditorUtilities::SaveAsset(UObject* Asset) const
{
	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(Asset->GetPackage());
	FEditorFileUtils::EPromptReturnCode ReturnCode;
	if (GIsAutomationTesting)
	{
		TGuardValue<bool> RunningUnattendedScriptGuard(GIsRunningUnattendedScript, true);
		ReturnCode = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false /*bCheckDirty*/, false /*bPromptToSave*/);
	}
	else
	{
		ReturnCode = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false /*bCheckDirty*/, false /*bPromptToSave*/);
	}
	return (ReturnCode == FEditorFileUtils::PR_Success);
}

bool UInterchangeEditorUtilities::IsRuntimeOrPIE() const
{
#if WITH_EDITOR
	return (GEditor && GEditor->PlayWorld) || GIsPlayInEditorWorld || IsRunningGame();
#else
	return true;
#endif // WITH_EDITOR
}

bool UInterchangeEditorUtilities::ClearEditorSelection() const
{
	if(GEditor)
	{
		GEditor->SelectNone(true, true, false);
	}
	return true;
}