// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeEditorUtilitiesModule.h"

#include "InterchangeEditorUtilities.h"
#include "InterchangeManager.h"


#define LOCTEXT_NAMESPACE "InterchangeEditorUtilities"

class FInterchangeEditorUtilitiesModule : public IInterchangeEditorUtilitiesModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

};

IMPLEMENT_MODULE(FInterchangeEditorUtilitiesModule, InterchangeEditorUtilities)

void FInterchangeEditorUtilitiesModule::StartupModule()
{
	UInterchangeManager::GetInterchangeManager().SetEditorUtilities(UInterchangeEditorUtilities::StaticClass());
}

void FInterchangeEditorUtilitiesModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
