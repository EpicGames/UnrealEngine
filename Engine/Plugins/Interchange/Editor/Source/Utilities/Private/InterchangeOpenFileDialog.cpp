// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeOpenFileDialog.h"

#include "DesktopPlatformModule.h"
#include "Factories/Factory.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "InterchangeManager.h"
#include "ObjectTools.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeOpenFileDialog)

namespace UE::Interchange::Utilities::Private
{
	FString GetOpenFileDialogExtensions(const TArray<FString>& TranslatorFormats, bool bShowAllFactoriesExtension, const TArray<FString>& ExtraFormats)
	{
		FString FileTypes;
		FString Extensions;
		TMultiMap<uint32, UFactory*> FilterIndexToFactory;

		if (bShowAllFactoriesExtension)
		{
			TArray<UFactory*> Factories;

			// Get the list of valid factories
			for (TObjectIterator<UClass> It; It; ++It)
			{
				UClass* CurrentClass = (*It);

				if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
				{
					UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
					if (Factory->bEditorImport)
					{
						Factories.Add(Factory);
					}
				}
			}

			
			// Generate the file types and extensions represented by the selected factories
			ObjectTools::GenerateFactoryFileExtensions(Factories, FileTypes, Extensions, FilterIndexToFactory);
		}

		ObjectTools::AppendFormatsFileExtensions(TranslatorFormats, FileTypes, Extensions, FilterIndexToFactory);
		
		if (!ExtraFormats.IsEmpty())
		{
			ObjectTools::AppendFormatsFileExtensions(ExtraFormats, FileTypes, Extensions, FilterIndexToFactory);
		}

		const FString FormatString = FString::Printf(TEXT("All Files (%s)|%s|%s"), *Extensions, *Extensions, *FileTypes);

		return FormatString;
	}

	bool FilePickerDialog(const FString& Extensions, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames)
	{
		// First, display the file open dialog for selecting the file.
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform)
		{
			FText PromptTitle = Parameters.Title.IsEmpty() ? NSLOCTEXT("InterchangeUtilities_OpenFileDialog", "FilePickerDialog", "Select a file") : Parameters.Title;

			const EFileDialogFlags::Type DialogFlags = Parameters.bAllowMultipleFiles ? EFileDialogFlags::Multiple : EFileDialogFlags::None;

			return DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				PromptTitle.ToString(),
				Parameters.DefaultPath,
				TEXT(""),
				*Extensions,
				DialogFlags,
				OutFilenames
			);
		}

		return false;
	}
} //ns UE::Interchange::Utilities::Private

bool UInterchangeFilePickerGeneric::FilePickerForTranslatorAssetType(const EInterchangeTranslatorAssetType TranslatorAssetType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames)
{
	FString Extensions = UE::Interchange::Utilities::Private::GetOpenFileDialogExtensions
	(
		UInterchangeManager::GetInterchangeManager().GetSupportedAssetTypeFormats(TranslatorAssetType), Parameters.bShowAllFactoriesExtension, Parameters.ExtraFormats
	);

	return UE::Interchange::Utilities::Private::FilePickerDialog(Extensions, Parameters, OutFilenames);
}

bool UInterchangeFilePickerGeneric::FilePickerForTranslatorType(const EInterchangeTranslatorType TranslatorType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames)
{
	FString Extensions = UE::Interchange::Utilities::Private::GetOpenFileDialogExtensions
	(
		UInterchangeManager::GetInterchangeManager().GetSupportedFormats(TranslatorType), Parameters.bShowAllFactoriesExtension, Parameters.ExtraFormats
	);
	return UE::Interchange::Utilities::Private::FilePickerDialog(Extensions, Parameters, OutFilenames);
}
