// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeEditorUtilitiesBase.h"

#include "InterchangeEditorUtilities.generated.h"

#define UE_API INTERCHANGEEDITORUTILITIES_API

UCLASS(MinimalAPI)
class UInterchangeEditorUtilities : public UInterchangeEditorUtilitiesBase
{
	GENERATED_BODY()

public:

protected:

	UE_API virtual bool SaveAsset(UObject* Asset) const override;

	UE_API virtual bool IsRuntimeOrPIE() const override;

	UE_API virtual bool ClearEditorSelection() const override;
};

#undef UE_API
