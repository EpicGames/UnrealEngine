// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeFilePickerBase.h"

#include "InterchangeOpenFileDialog.generated.h"

#define UE_API INTERCHANGEEDITORUTILITIES_API

UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UInterchangeFilePickerGeneric : public UInterchangeFilePickerBase
{
	GENERATED_BODY()

public:

protected:

	UE_API virtual bool FilePickerForTranslatorAssetType(const EInterchangeTranslatorAssetType TranslatorAssetType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames) override;
	UE_API virtual bool FilePickerForTranslatorType(const EInterchangeTranslatorType TranslatorType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames) override;
};

#undef UE_API
